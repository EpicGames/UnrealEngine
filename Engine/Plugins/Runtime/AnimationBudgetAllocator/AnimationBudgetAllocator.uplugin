{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Animation Budget Allocator",
	"Description": "Constrains the time taken for animation to run by dynamically throttling skeletal mesh component ticking.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AnimationBudgetAllocator",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"IsExperimentalVersion": false
}