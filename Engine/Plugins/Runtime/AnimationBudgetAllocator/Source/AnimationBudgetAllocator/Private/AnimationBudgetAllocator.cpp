// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationBudgetAllocator.h"
#include "Engine/Engine.h"
#include "IAnimationBudgetAllocatorModule.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "SkeletalMeshComponentBudgeted.h"
#include "AnimationBudgetAllocatorCVars.h"
#include "GameFramework/HUD.h"
#include "Debug/ReporterGraph.h"
#include "Engine/Canvas.h"

DECLARE_CYCLE_STAT(TEXT("InitialTick"), STAT_AnimationBudgetAllocator_Update, STATGROUP_AnimationBudgetAllocator);

DECLARE_DWORD_COUNTER_STAT(TEXT("Num Registered Components"), STAT_AnimationBudgetAllocator_NumRegisteredComponents, STATGROUP_AnimationBudgetAllocator);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Ticked Components"), STAT_AnimationBudgetAllocator_NumTickedComponents, STATGROUP_AnimationBudgetAllocator);

DECLARE_DWORD_COUNTER_STAT(TEXT("Demand"), STAT_AnimationBudgetAllocator_Demand, STATGROUP_AnimationBudgetAllocator);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Budget"), STAT_AnimationBudgetAllocator_Budget, STATGROUP_AnimationBudgetAllocator);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Average Work Unit (ms)"), STAT_AnimationBudgetAllocator_AverageWorkUnitTime, STATGROUP_AnimationBudgetAllocator);
DECLARE_DWORD_COUNTER_STAT(TEXT("Always Tick"), STAT_AnimationBudgetAllocator_AlwaysTick, STATGROUP_AnimationBudgetAllocator);
DECLARE_DWORD_COUNTER_STAT(TEXT("Throttled"), STAT_AnimationBudgetAllocator_Throttled, STATGROUP_AnimationBudgetAllocator);
DECLARE_DWORD_COUNTER_STAT(TEXT("Interpolated"), STAT_AnimationBudgetAllocator_Interpolated, STATGROUP_AnimationBudgetAllocator);
DECLARE_FLOAT_COUNTER_STAT(TEXT("SmoothedBudgetPressure"), STAT_AnimationBudgetAllocator_SmoothedBudgetPressure, STATGROUP_AnimationBudgetAllocator);

CSV_DEFINE_CATEGORY(AnimationBudget, true);

bool FAnimationBudgetAllocator::bCachedEnabled = false;

FAnimBudgetAllocatorComponentData::FAnimBudgetAllocatorComponentData(USkeletalMeshComponentBudgeted* InComponent, float InGameThreadLastTickTimeMs, int32 InStateChangeThrottle)
	: Component(InComponent)
	, RootPrerequisite(nullptr)
	, Significance(1.0f)
	, AccumulatedDeltaTime(0.0f)
	, GameThreadLastTickTimeMs(InGameThreadLastTickTimeMs)
	, GameThreadLastCompletionTimeMs(0.0f)
#if ENABLE_DRAW_DEBUG
	, PrevFrameGameThreadLastCompletionTimeMs(0.0f)
#endif
	, FrameOffset(0)
	, DesiredTickRate(1)
	, TickRate(1)
	, SkippedTicks(0)
	, StateChangeThrottle(InStateChangeThrottle)
	, bTickEnabled(true)
	, bAlwaysTick(false)
	, bTickEvenIfNotRendered(false)
	, bInterpolate(false)
	, bReducedWork(false)
	, bAllowReducedWork(true)
	, bAutoCalculateSignificance(false)
	, bOnScreen(false)
	, bWasOffScreenPrevTick(false)
	, bNeverThrottle(true)
{}

FAnimationBudgetAllocator::FAnimationBudgetAllocator(UWorld* InWorld)
	: World(InWorld)
	, AverageWorkUnitTimeMs(GBudgetParameters.InitialEstimatedWorkUnitTimeMs)
	, NumComponentsToNotSkip(0)
	, TotalEstimatedTickTimeMs(0.0f)
	, NumWorkUnitsForAverage(0.0f)
	, SmoothedBudgetPressure(0.0f)
#if ENABLE_DRAW_DEBUG
	, DebugTotalTime(0.0f)
	, CurrentDebugTimeDisplay(0.0f)
	, DebugSmoothedTotalTime(0.0f)
#endif
	, ReducedComponentWorkCounter(0)
	, CurrentFrameOffset(0)
	, bEnabled(false)
	, bCurrentlyUpdating(false)
	, bHasBegunPlay(InWorld->HasBegunPlay())
{
	FAnimationBudgetAllocator::bCachedEnabled = GAnimationBudgetEnabled == 1;
	
	SetParametersFromCVars();

	OnWorldBeginPlayHandle = InWorld->OnWorldBeginPlay.AddRaw(this, &FAnimationBudgetAllocator::HandleWorldBeginPlay);
	GarbageCollectStartedHandle = FCoreUObjectDelegates::GetGarbageCollectStartedDelegate().AddRaw(this, &FAnimationBudgetAllocator::HandleGarbageCollectStarted);
	PostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FAnimationBudgetAllocator::HandlePostGarbageCollect);
	OnWorldPreActorTickHandle = FWorldDelegates::OnWorldPreActorTick.AddRaw(this, &FAnimationBudgetAllocator::OnWorldPreActorTick);
	OnCVarParametersChangedHandle = GOnCVarParametersChanged.AddRaw(this, &FAnimationBudgetAllocator::SetParametersFromCVars);
#if ENABLE_DRAW_DEBUG
	OnHUDPostRenderHandle = AHUD::OnHUDPostRender.AddRaw(this, &FAnimationBudgetAllocator::OnHUDPostRender);
#endif
}

FAnimationBudgetAllocator::~FAnimationBudgetAllocator()
{
#if ENABLE_DRAW_DEBUG
	AHUD::OnHUDPostRender.Remove(OnHUDPostRenderHandle);
#endif
	if(World)
	{
		World->OnWorldBeginPlay.Remove(OnWorldBeginPlayHandle);
		World = nullptr;
	}

	FCoreUObjectDelegates::GetGarbageCollectStartedDelegate().Remove(GarbageCollectStartedHandle);
	FCoreUObjectDelegates::GetPostGarbageCollect().Remove(PostGarbageCollectHandle);
	FWorldDelegates::OnWorldPreActorTick.Remove(OnWorldPreActorTickHandle);
	GOnCVarParametersChanged.Remove(OnCVarParametersChangedHandle);
}

IAnimationBudgetAllocator* IAnimationBudgetAllocator::Get(UWorld* InWorld)
{
	return IAnimationBudgetAllocatorModule::Get(InWorld);
}

void FAnimationBudgetAllocator::SetComponentTickEnabled(USkeletalMeshComponentBudgeted* Component, bool bShouldTick)
{
	checkSlow(World == Component->GetWorld());

	if (FAnimationBudgetAllocator::bCachedEnabled && bEnabled)
	{
		int32 Handle = Component->GetAnimationBudgetHandle();
		if(Handle != INDEX_NONE)
		{
			AllComponentData[Handle].bTickEnabled = bShouldTick;
		}

		TickEnableHelper(Component, bShouldTick);
	}
	else
	{
		TickEnableHelper(Component, bShouldTick);
	}
}

bool FAnimationBudgetAllocator::IsComponentTickEnabled(USkeletalMeshComponentBudgeted* Component) const
{
	checkSlow(World == Component->GetWorld());

	if (FAnimationBudgetAllocator::bCachedEnabled && bEnabled)
	{
		int32 Handle = Component->GetAnimationBudgetHandle();
		if(Handle != INDEX_NONE)
		{
			return AllComponentData[Handle].bTickEnabled;
		}

		return Component->PrimaryComponentTick.IsTickFunctionEnabled();
	}
	else
	{
		return Component->PrimaryComponentTick.IsTickFunctionEnabled();
	}
}

void FAnimationBudgetAllocator::SetComponentSignificance(USkeletalMeshComponentBudgeted* Component, float Significance, bool bAlwaysTick, bool bTickEvenIfNotRendered, bool bAllowReducedWork, bool bNeverThrottle)
{
	checkSlow(World == Component->GetWorld());

	int32 Handle = Component->GetAnimationBudgetHandle();
	if(Handle != INDEX_NONE)
	{
		FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[Handle];
		ComponentData.Significance = Significance;
		ComponentData.bAlwaysTick = bAlwaysTick;
		ComponentData.bTickEvenIfNotRendered = bTickEvenIfNotRendered;
		ComponentData.bAllowReducedWork = !bAlwaysTick && bAllowReducedWork;	// Dont allow reduced work if we are set to 'always tick'
		ComponentData.bNeverThrottle = bNeverThrottle;
	}
}

void FAnimationBudgetAllocator::TickEnableHelper(USkeletalMeshComponent* InComponent, bool bInEnable)
{
	InComponent->PrimaryComponentTick.SetTickFunctionEnable(bInEnable);
}

void FAnimationBudgetAllocator::QueueSortedComponentIndices(float InDeltaSeconds)
{
	const float WorldTime = World->TimeSeconds - 1.0f;

	NumComponentsToNotSkip = 0;
	NumComponentsToNotThrottle = 0;
	TotalEstimatedTickTimeMs = 0.0f;
	NumWorkUnitsForAverage = 0.0f;

	const bool bUseSignificanceDelegate = USkeletalMeshComponentBudgeted::OnCalculateSignificance().IsBound();

	auto QueueComponentTick = [InDeltaSeconds, this, bUseSignificanceDelegate](FAnimBudgetAllocatorComponentData& InComponentData, int32 InComponentIndex, bool bInOnScreen)
	{
		InComponentData.AccumulatedDeltaTime += InDeltaSeconds;

		InComponentData.bWasOffScreenPrevTick = (bInOnScreen && !InComponentData.bOnScreen);

		InComponentData.bOnScreen = bInOnScreen;
		InComponentData.StateChangeThrottle = InComponentData.StateChangeThrottle < 0 ? InComponentData.StateChangeThrottle : InComponentData.StateChangeThrottle - 1;

		if(InComponentData.bAlwaysTick)
		{
			NumComponentsToNotSkip++;
		}
		else if(InComponentData.bNeverThrottle)
		{
			NumComponentsToNotThrottle++;
		}

		// Accumulate average tick time
		TotalEstimatedTickTimeMs += InComponentData.GameThreadLastTickTimeMs + InComponentData.GameThreadLastCompletionTimeMs;
		NumWorkUnitsForAverage += 1.0f;

		AllSortedComponentData.Add(InComponentIndex);
#if WITH_TICK_DEBUG
		AllSortedComponentDataDebug.Add(&InComponentData);
#endif

		// Auto-calculate significance here if we are set to
		if(InComponentData.bAutoCalculateSignificance)
		{
			float Significance;
			if(bUseSignificanceDelegate)
			{
				Significance = USkeletalMeshComponentBudgeted::OnCalculateSignificance().Execute(InComponentData.Component);
			}
			else
			{
				Significance = InComponentData.Component->GetDefaultSignificance();
			}
			
			SetComponentSignificance(InComponentData.Component,
				Significance,
				InComponentData.bAlwaysTick,
				InComponentData.bTickEvenIfNotRendered,
				InComponentData.bAllowReducedWork,
				InComponentData.bNeverThrottle);
		}
	};

	auto DisableComponentTick = [this](FAnimBudgetAllocatorComponentData& InComponentData)
	{
		InComponentData.SkippedTicks = 0;
		InComponentData.AccumulatedDeltaTime = 0.0f;

		// Re-distribute frame offsets for components that wont be ticked, to try to 'level' the distribution
		InComponentData.FrameOffset = CurrentFrameOffset++;

		TickEnableHelper(InComponentData.Component, false);
	};

	uint8 MaxComponentTickFunctionIndex = 0;
	int32 ComponentIndex = 0;
	for (FAnimBudgetAllocatorComponentData& ComponentData : AllComponentData)
	{
		ComponentData.bReducedWorkDueToOffscreen = false;

		if(ComponentData.bTickEnabled)
		{
			USkeletalMeshComponentBudgeted* Component = ComponentData.Component;
			if (Component && Component->IsRegistered())
			{
				auto ShouldComponentTick = [WorldTime](const USkeletalMeshComponentBudgeted* InComponent, const FAnimBudgetAllocatorComponentData& InComponentData)
				{
					return (((InComponent->bUseScreenRenderStateForUpdate ? InComponent->GetLastRenderTimeOnScreen() : InComponent->GetLastRenderTime()) > WorldTime) ||
						(InComponent->GetShouldUseActorRenderedFlag() && InComponent->GetAttachmentRootActor() && InComponent->GetAttachmentRootActor()->WasRecentlyRendered())  ||
							InComponentData.bTickEvenIfNotRendered ||
							InComponent->ShouldTickPose() ||
							InComponent->ShouldUpdateTransform(false) ||	// We can force this to false, only used in WITH_EDITOR
							InComponent->VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::AlwaysTickPose ||
							InComponent->VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones);
				};

				// Whether or not we will tick
				bool bShouldTick = ShouldComponentTick(Component, ComponentData);

				// Avoid ticking when root prerequisites dont tick (assumes leader pose or copy pose relationship)
				if(bShouldTick && ComponentData.RootPrerequisite != nullptr)
				{
					const int32 PrerequisiteHandle = ComponentData.RootPrerequisite->GetAnimationBudgetHandle();
					if(PrerequisiteHandle != INDEX_NONE)
					{
						const FAnimBudgetAllocatorComponentData& RootPrerequisiteComponentData = AllComponentData[PrerequisiteHandle];
						bShouldTick &= ShouldComponentTick(ComponentData.RootPrerequisite, RootPrerequisiteComponentData);
					}
				}

				if(bShouldTick)
				{
					// Push into a separate limited list if we are 'tick even if not rendered'.
					// Skip offscreen components with a significance of zero or less (or less than zero if GAnimationBudgetComponentsWithZeroSigAreNonRendered is enabled).
					if((Component->bUseScreenRenderStateForUpdate ? Component->GetLastRenderTimeOnScreen() : Component->GetLastRenderTime()) <= WorldTime && 
						ComponentData.bTickEvenIfNotRendered && 
						((GAnimationBudgetComponentsWithZeroSigAreNonRendered && ComponentData.Significance >= 0.0f) || (!GAnimationBudgetComponentsWithZeroSigAreNonRendered && ComponentData.Significance > 0.0f)))
					{
						NonRenderedComponentData.Add(ComponentIndex);
					}
					else
					{
						QueueComponentTick(ComponentData, ComponentIndex, true);
					}
				}
				else
				{
					DisableComponentTick(ComponentData);

#if ENABLE_DRAW_DEBUG
					NonRenderedNotSignificantComponentDataForDebugDraw.Add(ComponentIndex);
#endif // #if ENABLE_DRAW_DEBUG
				}
			}
		}
#if ENABLE_DRAW_DEBUG
		else
		{
			DisabledTickingComponentDataForDebugDraw.Add(ComponentIndex);
		}
#endif // #if ENABLE_DRAW_DEBUG


		if(ComponentData.bReducedWork)
		{
			if(!ComponentData.bAllowReducedWork)
			{
				DisallowedReducedWorkComponentData.Add(ComponentIndex);
			}
			else
			{
				ReducedWorkComponentData.Add(ComponentIndex);
			}
		}

		ComponentIndex++;
	}

	// Sort by significance, largest first
	auto SignificanceSortPredicate = [this](int32 InIndex0, int32 InIndex1)
	{
		const FAnimBudgetAllocatorComponentData& ComponentData0 = AllComponentData[InIndex0];
		const FAnimBudgetAllocatorComponentData& ComponentData1 = AllComponentData[InIndex1];

		// prioritize component with always tick enabled
		if (ComponentData0.bAlwaysTick != ComponentData1.bAlwaysTick)
		{
			return static_cast<bool>(ComponentData0.bAlwaysTick);
		}

		return ComponentData0.Significance > ComponentData1.Significance;
	};

	AllSortedComponentData.Sort(SignificanceSortPredicate);
	ReducedWorkComponentData.Sort(SignificanceSortPredicate);
	NonRenderedComponentData.Sort(SignificanceSortPredicate);

	const int32 MaxOffscreenComponents = FMath::Min(NonRenderedComponentData.Num(), Parameters.MaxTickedOffsreenComponents);
	if(MaxOffscreenComponents > 0)
	{
		auto ReduceWorkForOffscreenComponent = [](FAnimBudgetAllocatorComponentData& InComponentData)
		{
			if(InComponentData.bAllowReducedWork
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				&& !GAnimationBudgetDebugForce
#endif
			)
			{
				InComponentData.bReducedWorkDueToOffscreen = true;

				if (!InComponentData.bReducedWork && InComponentData.Component->OnReduceWork().IsBound())
				{
#if WITH_TICK_DEBUG
					UE_LOGF(LogTemp, Warning, "Force-decreasing offscreen component work (mesh %ls) (actor %llx)", InComponentData.Component->SkeletalMesh ? *InComponentData.Component->SkeletalMesh->GetName() : TEXT("null"), (uint64)InComponentData.Component->GetOwner());
#endif
					InComponentData.Component->OnReduceWork().Execute(InComponentData.Component, true);
					InComponentData.bReducedWork = true;
				}
			}
		};

		// Queue first N offscreen ticks
		int32 NonRenderedComponentIndex = 0;
		for(; NonRenderedComponentIndex < MaxOffscreenComponents; ++NonRenderedComponentIndex)
		{
			FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[NonRenderedComponentData[NonRenderedComponentIndex]];
			QueueComponentTick(ComponentData, NonRenderedComponentData[NonRenderedComponentIndex], false);

			// Always move to reduced work offscreen
			ReduceWorkForOffscreenComponent(ComponentData);

			// Offscreen will need state changing ASAP when back onscreen
			ComponentData.StateChangeThrottle = -1;
		}

		// Disable ticks for the rest
		for(; NonRenderedComponentIndex < NonRenderedComponentData.Num(); ++NonRenderedComponentIndex)
		{
			FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[NonRenderedComponentData[NonRenderedComponentIndex]];
			DisableComponentTick(AllComponentData[NonRenderedComponentData[NonRenderedComponentIndex]]);

			// Always move to reduced work offscreen
			ReduceWorkForOffscreenComponent(ComponentData);

			// Offscreen will need state changing ASAP when back onscreen
			ComponentData.StateChangeThrottle = -1;
		}

		if (GAnimationBudgetMaintainReduceWorkWhenOffScreen)
		{
			// if component ticks were disabled either here or through reduce work, they should be taken out of the sorted component list
			for (int32 SortedCompIdx = 0 ; SortedCompIdx < AllSortedComponentData.Num() ; ++SortedCompIdx)
			{
				FAnimBudgetAllocatorComponentData& SortedComponentData = AllComponentData[AllSortedComponentData[SortedCompIdx]];
				if (!SortedComponentData.bTickEnabled)
				{
					AllSortedComponentData.RemoveAt(SortedCompIdx);
					--SortedCompIdx;
				}
			}
		}

		// re-sort now we have inserted offscreen components
		AllSortedComponentData.Sort(SignificanceSortPredicate);
	}

#if WITH_TICK_DEBUG
	auto SignificanceDebugSortPredicate = [](const FAnimBudgetAllocatorComponentData& InComponentData0, const FAnimBudgetAllocatorComponentData& InComponentData1)
	{
		return InComponentData0.Significance > InComponentData1.Significance;
	};

	AllSortedComponentDataDebug.Sort(SignificanceDebugSortPredicate);
#endif
}

int32 FAnimationBudgetAllocator::CalculateWorkDistributionAndQueue(float InDeltaSeconds, float& OutAverageTickRate)
{
	int32 NumTicked = 0;

#if ENABLE_DRAW_DEBUG
	bEmergencyReducedWorkActive = false;
#endif

	auto QueueForTick = [&NumTicked, this](FAnimBudgetAllocatorComponentData& InComponentData, int32 InStateChangeThrottleInFrames)
	{
		const int32 PrerequisiteHandle = InComponentData.RootPrerequisite != nullptr ? InComponentData.RootPrerequisite->GetAnimationBudgetHandle() : INDEX_NONE;
		const FAnimBudgetAllocatorComponentData& ComponentDataToCheck = PrerequisiteHandle != INDEX_NONE ? AllComponentData[PrerequisiteHandle] : InComponentData;

		// Using (frame offset + frame counter) % tick rate allows us to only tick at the specified interval,
		// but at a roughly even distribution over all registered components
		bool bTickThisFrame = (((GFrameCounter + ComponentDataToCheck.FrameOffset) % ComponentDataToCheck.TickRate) == 0);

		if (GAnimationBudgetForceTickWhenComponentExitsOffScreen != 0)
		{
			if (InComponentData.bWasOffScreenPrevTick && !bTickThisFrame)
			{
				// force tick this frame and fix up frameoffset so we maintain the same tick rate
				bTickThisFrame = true;
				InComponentData.FrameOffset += (InComponentData.TickRate - ((GFrameCounter + InComponentData.FrameOffset) % InComponentData.TickRate));
			}
		}

		// Calculate interp alpha even when interpolation or ticking is disabled as this is used to decide how much root motion to consume
		// each frame by character movement
		InComponentData.SkippedTicks = bTickThisFrame ? 0 : (InComponentData.SkippedTicks + 1);
		
		// Each frame we replace the source pose with the interpolated pose, so Alpha is set to blend from the current pose, one tick closer to the final pose.
		const int TicksRemaining = InComponentData.TickRate - InComponentData.SkippedTicks;
		float Alpha = (InComponentData.TickRate > 1) ? FMath::Clamp((1.0f / TicksRemaining), 0.0f, 1.0f) : 1.f;
		InComponentData.Component->SetExternalInterpolationAlpha(Alpha);

#if ENABLE_DRAW_DEBUG
		InComponentData.PrevFrameGameThreadLastCompletionTimeMs = InComponentData.GameThreadLastCompletionTimeMs;
#endif

		if((ComponentDataToCheck.bInterpolate && ComponentDataToCheck.bOnScreen) || bTickThisFrame)
		{
			InComponentData.bInterpolate = ComponentDataToCheck.bInterpolate;

			// Reset completion time as it may not always be run
			InComponentData.GameThreadLastCompletionTimeMs = 0.0f;

			InComponentData.Component->EnableExternalInterpolation(InComponentData.TickRate > 1 && InComponentData.bInterpolate);
			InComponentData.Component->EnableExternalUpdate(bTickThisFrame);
			InComponentData.Component->EnableExternalEvaluationRateLimiting(InComponentData.TickRate > 1);
			AActor* MyOwner = InComponentData.Component->GetOwner();
			const float CustomTimeDialation = MyOwner ? MyOwner->CustomTimeDilation : 1.0f;
			InComponentData.Component->SetExternalDeltaTime(InComponentData.AccumulatedDeltaTime * CustomTimeDialation);
			InComponentData.Component->SetExternalTickRate(InComponentData.TickRate);

			InComponentData.AccumulatedDeltaTime = bTickThisFrame ? 0.0f : InComponentData.AccumulatedDeltaTime;

			TickEnableHelper(InComponentData.Component, true);

			// Only switch to desired tick rate when we actually tick (throttled)
			if(bTickThisFrame && InComponentData.StateChangeThrottle < 0)
			{
				InComponentData.TickRate = InComponentData.DesiredTickRate;
				InComponentData.StateChangeThrottle = InStateChangeThrottleInFrames;
			}

			NumTicked++;
		}
		else
		{
			// Disable external update here as character movement 'manual ticks' need to respect the external control as well
			// otherwise they will end up ticking too often, ending up with incorrect root motion
			InComponentData.Component->EnableExternalUpdate(false);

			TickEnableHelper(InComponentData.Component, false);
		}
	};

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if(GAnimationBudgetDebugForce)
	{
		for(int32 SortedComponentIndex : AllSortedComponentData)
		{
			FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[SortedComponentIndex];

			if (GAnimationBudgetDebugForceExcludeAlwaysTicked == 0 || !ComponentData.bAlwaysTick)
			{
				ComponentData.TickRate = ComponentData.DesiredTickRate = GAnimationBudgetDebugForceRate;
				ComponentData.bInterpolate = !!GAnimationBudgetDebugForceInterpolation;
				if (ComponentData.Component->OnReduceWork().IsBound())
				{
					if (ComponentData.bReducedWork && !GAnimationBudgetDebugForceReducedWork)
					{
						ComponentData.Component->OnReduceWork().Execute(ComponentData.Component, false);
						ComponentData.bReducedWork = false;
					}
					else if (!ComponentData.bReducedWork && GAnimationBudgetDebugForceReducedWork)
					{
						ComponentData.Component->OnReduceWork().Execute(ComponentData.Component, true);
						ComponentData.bReducedWork = true;
					}
				}
			}
			else
			{
				ComponentData.TickRate = 1;
				ComponentData.DesiredTickRate = 1;
				ComponentData.bInterpolate = false;
			}

			ComponentData.StateChangeThrottle = -1;
			QueueForTick(ComponentData, -1);
		}
	}
	else
#endif
	{
		const int32 TotalIdealWorkUnits = AllSortedComponentData.Num();

		SET_DWORD_STAT(STAT_AnimationBudgetAllocator_Demand, TotalIdealWorkUnits);

		if(TotalIdealWorkUnits > 0)
		{
			// Calc smoothed average of last frames' work units
			const float AverageTickTimeMs = TotalEstimatedTickTimeMs / NumWorkUnitsForAverage;
			AverageWorkUnitTimeMs = FMath::FInterpTo(AverageWorkUnitTimeMs, AverageTickTimeMs, InDeltaSeconds, Parameters.WorkUnitSmoothingSpeed);

			SET_FLOAT_STAT(STAT_AnimationBudgetAllocator_AverageWorkUnitTime, AverageWorkUnitTimeMs);
			CSV_CUSTOM_STAT(AnimationBudget, AverageWorkUnitTimeMs, AverageTickTimeMs, ECsvCustomStatOp::Set);

			// Want to map the remaining (non-fixed) work units so that we only execute N work units per frame.
			// If we can go over budget to keep quality then we use that value

			const float AvailableBudget = Parameters.BudgetInMs - ExternalAnimationTimeSpent;
			ExternalAnimationTimeSpent = 0.0f;

			const float WorkUnitBudget = FMath::Max(AvailableBudget / AverageWorkUnitTimeMs, (float)TotalIdealWorkUnits * Parameters.MinQuality);

			SET_FLOAT_STAT(STAT_AnimationBudgetAllocator_Budget, WorkUnitBudget);

			// Ramp-off work units that we tick every frame once required ticks start exceeding budget
			const float WorkUnitsExcess = FMath::Max(0.0f, TotalIdealWorkUnits - WorkUnitBudget);
			const float WorkUnitsToRunInFull = FMath::Clamp(WorkUnitBudget - (WorkUnitsExcess * Parameters.AlwaysTickFalloffAggression), (float)NumComponentsToNotSkip, (float)TotalIdealWorkUnits);
			SET_DWORD_STAT(STAT_AnimationBudgetAllocator_AlwaysTick, WorkUnitsToRunInFull);
			BUDGET_CSV_STAT(AnimationBudget, NumAlwaysTicked, WorkUnitsToRunInFull, ECsvCustomStatOp::Set);
			const int32 FullIndexEnd = (int32)WorkUnitsToRunInFull;

			// Account for the actual time that we think the fixed ticks will take
			// This works better when budget to work unit ratio is low
			float FullTickTime = 0.0f;
			int32 SortedComponentIndex;
			for (SortedComponentIndex = 0; SortedComponentIndex < FullIndexEnd; ++SortedComponentIndex)
			{
				FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[AllSortedComponentData[SortedComponentIndex]];
				FullTickTime += ComponentData.GameThreadLastCompletionTimeMs + ComponentData.GameThreadLastTickTimeMs;
			}

			float FullTickWorkUnits = FMath::Min(FullTickTime / AverageWorkUnitTimeMs, WorkUnitsToRunInFull);

			float RemainingBudget = FMath::Max(0.0f, WorkUnitBudget - FullTickWorkUnits);
			float RemainingWorkUnitsToRun = FMath::Max(0.0f, TotalIdealWorkUnits - FullTickWorkUnits);

			// Ramp off interpolated units in a similar way
			const float WorkUnitsToInterpolate = FMath::Min(FMath::Max(RemainingBudget - (WorkUnitsExcess * Parameters.InterpolationFalloffAggression), (float)FMath::Min(Parameters.MaxInterpolatedComponents, NumComponentsToNotThrottle)), RemainingWorkUnitsToRun);
			SET_DWORD_STAT(STAT_AnimationBudgetAllocator_Interpolated, WorkUnitsToInterpolate);

			const int32 InterpolationIndexEnd = FMath::Min((int32)WorkUnitsToInterpolate + (int32)WorkUnitsToRunInFull, TotalIdealWorkUnits);

			const float MaxInterpolationRate = (float)Parameters.InterpolationMaxRate;

			// Calc remaining (throttled) work units
			RemainingBudget = FMath::Max(0.0f, RemainingBudget - (WorkUnitsToInterpolate * Parameters.InterpolationTickMultiplier));
			RemainingWorkUnitsToRun = FMath::Max(0.0f, RemainingWorkUnitsToRun - (WorkUnitsToInterpolate * Parameters.InterpolationTickMultiplier));

			SET_DWORD_STAT(STAT_AnimationBudgetAllocator_Throttled, RemainingWorkUnitsToRun);

			// Midpoint of throttle gradient is RemainingWorkUnitsToRun / RemainingBudget.
			// If we distributed this as a constant we would get each component ticked
			// at the same rate. However we want to tick more significant meshes more often,
			// so we keep the area under the curve constant and intercept the line with this centroid.
			// Care must be taken with rounding to keep workload in-budget.
			const float ThrottleRateDenominator = RemainingBudget > 1.0f ? RemainingBudget : 1.0f;
			const float MaxThrottleRate = FMath::Min(FMath::CeilToFloat(FMath::Max(1.0f, RemainingWorkUnitsToRun / ThrottleRateDenominator) * 2.0f), (float)Parameters.MaxTickRate);
			const float ThrottleDenominator = RemainingWorkUnitsToRun > 0.0f ? RemainingWorkUnitsToRun : 1.0f;

			// Bucket 1: always ticked
			for (SortedComponentIndex = 0; SortedComponentIndex < FullIndexEnd; ++SortedComponentIndex)
			{
				FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[AllSortedComponentData[SortedComponentIndex]];

				// not skipping frames here as we can either match demand or these components need a full update
				ComponentData.TickRate = 1;
				ComponentData.DesiredTickRate = 1;
				ComponentData.bInterpolate = false;
			}

			// Bucket 2: interpolated
			int32 NumInterpolated = 0;
			for (SortedComponentIndex = FullIndexEnd; SortedComponentIndex < InterpolationIndexEnd; ++SortedComponentIndex)
			{
				FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[AllSortedComponentData[SortedComponentIndex]];

				const float Alpha = (((float)SortedComponentIndex - FullIndexEnd) / WorkUnitsToInterpolate);
				ComponentData.DesiredTickRate = FMath::Min((int32)FMath::FloorToFloat(FMath::Lerp(2.0f, MaxInterpolationRate, Alpha) + 0.5f), 255);
				ComponentData.bInterpolate = true;
				NumInterpolated++;
			}

			// Bucket 3: Rate limited
			int32 NumThrottled = 0;
			for (SortedComponentIndex = InterpolationIndexEnd; SortedComponentIndex < TotalIdealWorkUnits; ++SortedComponentIndex)
			{
				FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[AllSortedComponentData[SortedComponentIndex]];

				const float Alpha = (((float)SortedComponentIndex - InterpolationIndexEnd) / ThrottleDenominator);
				ComponentData.DesiredTickRate = FMath::Min((int32)FMath::FloorToFloat(FMath::Lerp(2.0f, MaxThrottleRate, Alpha) + 0.5f), 255);
				ComponentData.bInterpolate = false;
				NumThrottled++;
			}

			BUDGET_CSV_STAT(AnimationBudget, NumInterpolated, NumInterpolated, ECsvCustomStatOp::Set);
			BUDGET_CSV_STAT(AnimationBudget, NumThrottled, RemainingWorkUnitsToRun, ECsvCustomStatOp::Set);

			const float BudgetPressure = (float)TotalIdealWorkUnits / WorkUnitBudget;
			SmoothedBudgetPressure = FMath::FInterpTo(SmoothedBudgetPressure, BudgetPressure, InDeltaSeconds, Parameters.BudgetPressureSmoothingSpeed);

			float BudgetPressureInterpAlpha = FMath::Clamp((SmoothedBudgetPressure - Parameters.BudgetFactorBeforeAggressiveReducedWork) * 0.5f, 0.0f, 1.0f);
			int32 StateChangeThrottleInFrames = (int32)FMath::Lerp(4.0f, (float)Parameters.StateChangeThrottleInFrames, BudgetPressureInterpAlpha);

			SET_FLOAT_STAT(STAT_AnimationBudgetAllocator_SmoothedBudgetPressure, SmoothedBudgetPressure);

			const bool bEmergencyReducedWork = (SmoothedBudgetPressure >= Parameters.BudgetPressureBeforeEmergencyReducedWork) && (GAnimationBudgetEmergencyReduceWorkDisabled == 0);

#if ENABLE_DRAW_DEBUG
			bEmergencyReducedWorkActive = bEmergencyReducedWork;
#endif

			// Check all datas to line up tick rates based on prerequisites before queuing for ticks so ticks can be consistent
			for (SortedComponentIndex = 0; SortedComponentIndex < TotalIdealWorkUnits; ++SortedComponentIndex)
			{
				FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[AllSortedComponentData[SortedComponentIndex]];

				// Ensure that root prerequisite doesnt end up with a lower (or different) tick rate than dependencies
				if (ComponentData.RootPrerequisite != nullptr)
				{
					const int32 PrerequisiteHandle = ComponentData.RootPrerequisite->GetAnimationBudgetHandle();
					if (PrerequisiteHandle != INDEX_NONE)
					{
						FAnimBudgetAllocatorComponentData& RootPrerequisiteComponentData = AllComponentData[PrerequisiteHandle];
						RootPrerequisiteComponentData.TickRate = ComponentData.TickRate = FMath::Min(ComponentData.TickRate, RootPrerequisiteComponentData.TickRate);
						RootPrerequisiteComponentData.DesiredTickRate = ComponentData.DesiredTickRate = FMath::Min(ComponentData.DesiredTickRate, RootPrerequisiteComponentData.DesiredTickRate);
						RootPrerequisiteComponentData.StateChangeThrottle = ComponentData.StateChangeThrottle = FMath::Min(ComponentData.StateChangeThrottle, RootPrerequisiteComponentData.StateChangeThrottle);
					}
				}
			}

			// Queue for tick
			for (SortedComponentIndex = 0; SortedComponentIndex < TotalIdealWorkUnits; ++SortedComponentIndex)
			{
				FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[AllSortedComponentData[SortedComponentIndex]];

				QueueForTick(ComponentData, StateChangeThrottleInFrames);
			}

			// If any components are not longer allowed to perform reduced work, force them back out
			for(int32 DisallowedReducedWorkComponentIndex : DisallowedReducedWorkComponentData)
			{
				FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[DisallowedReducedWorkComponentIndex];
				if(ComponentData.bReducedWork && ComponentData.Component->OnReduceWork().IsBound())
				{
#if WITH_TICK_DEBUG
					UE_LOGF(LogTemp, Warning, "Force-increasing component work (mesh %ls) (actor %llx)", ComponentData.Component->SkeletalMesh ? *ComponentData.Component->SkeletalMesh->GetName() : TEXT("null"), (uint64)ComponentData.Component->GetOwner());
#endif
					ComponentData.Component->OnReduceWork().Execute(ComponentData.Component, false);
					ComponentData.bReducedWork = false;
				}
			}

			if(--ReducedComponentWorkCounter <= 0)
			{
				// Scale num components to switch based on budget pressure
				const int32 NumComponentsToSwitch = (int32)FMath::Lerp(1.0f, (float)Parameters.ReducedWorkThrottleMaxPerFrame, BudgetPressureInterpAlpha);
				int32 ComponentsSwitched = 0;

				// If we have any components running reduced work when we have an excess, then move them out of the 'reduced' pool per tick
				if (ReducedWorkComponentData.Num() > 0 && SmoothedBudgetPressure < Parameters.BudgetFactorBeforeReducedWork - Parameters.BudgetFactorBeforeReducedWorkEpsilon)
				{
					for(int32 ReducedWorkComponentIndex : ReducedWorkComponentData)
					{
						FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[ReducedWorkComponentIndex];

						// check if ReducedWorkDueToOffscreen is true from this component or the root component. If so, don't remove reduce work due to budget
						bool bReducedWorkDueToOffscreen = false;
						
						if (GAnimationBudgetMaintainReduceWorkWhenOffScreen)
						{
							bReducedWorkDueToOffscreen = ComponentData.bReducedWorkDueToOffscreen;
							if (ComponentData.RootPrerequisite != nullptr)
							{
								const int32 PrerequisiteHandle = ComponentData.RootPrerequisite->GetAnimationBudgetHandle();
								if (PrerequisiteHandle != INDEX_NONE)
								{
									FAnimBudgetAllocatorComponentData& RootPrerequisiteComponentData = AllComponentData[PrerequisiteHandle];
									bReducedWorkDueToOffscreen |= RootPrerequisiteComponentData.bReducedWorkDueToOffscreen;
								}
							}
						}

						if(ComponentData.bReducedWork && !bReducedWorkDueToOffscreen && ComponentData.Component->OnReduceWork().IsBound())
						{
#if WITH_TICK_DEBUG
							UE_LOGF(LogTemp, Warning, "Increasing component work (mesh %ls) (actor %llx)", ComponentData.Component->SkeletalMesh ? *ComponentData.Component->SkeletalMesh->GetName() : TEXT("null"), (uint64)ComponentData.Component->GetOwner());
#endif
							ComponentData.Component->OnReduceWork().Execute(ComponentData.Component, false);
							ComponentData.bReducedWork = false;

							ComponentsSwitched++;
							if(ComponentsSwitched >= NumComponentsToSwitch)
							{
								break;	
							}
						}
					}
				}
				else if(SmoothedBudgetPressure > Parameters.BudgetFactorBeforeReducedWork)
				{
					// Any work units that we interpolate or throttle should also be eligible for work reduction (which can involve disabling other ticks), so set them all now if needed
					for (SortedComponentIndex = TotalIdealWorkUnits - 1; SortedComponentIndex >= FullIndexEnd; --SortedComponentIndex)
					{
						FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[AllSortedComponentData[SortedComponentIndex]];

						const bool bAllowReducedWork = (ComponentData.bAllowReducedWork || bEmergencyReducedWork) && !ComponentData.bAlwaysTick;

						if(bAllowReducedWork && !ComponentData.bReducedWork && ComponentData.Component->OnReduceWork().IsBound())
						{
#if WITH_TICK_DEBUG
							UE_LOGF(LogTemp, Warning, "Reducing component work (mesh %ls) (actor %llx)", ComponentData.Component->SkeletalMesh ? *ComponentData.Component->SkeletalMesh->GetName() : TEXT("null"), (uint64)ComponentData.Component->GetOwner());
#endif
							ComponentData.Component->OnReduceWork().Execute(ComponentData.Component, true);
							ComponentData.bReducedWork = true;

							ComponentsSwitched++;
							if(ComponentsSwitched >= NumComponentsToSwitch)
							{
								break;	
							}
						}
					}
				}

				// Scale the rate at which we consider reducing component work based on budget pressure
				ReducedComponentWorkCounter = (int32)FMath::Lerp((float)Parameters.ReducedWorkThrottleMaxInFrames, (float)Parameters.ReducedWorkThrottleMinInFrames, BudgetPressureInterpAlpha);
			}
		}

#if CSV_PROFILER_STATS
		if(AllSortedComponentData.Num() > 0)
		{
			for (int32 ComponentDataIndex : AllSortedComponentData)
			{
				FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[ComponentDataIndex];
				OutAverageTickRate += (float)ComponentData.TickRate;
			}

			OutAverageTickRate /= (float)AllSortedComponentData.Num();
		}
#endif
	}

	return NumTicked;
}

void FAnimationBudgetAllocator::OnWorldPreActorTick(UWorld* InWorld, ELevelTick InLevelTick, float InDeltaSeconds)
{
	if(World == InWorld && InLevelTick == LEVELTICK_All)
	{
		Update(InDeltaSeconds);
	}
}

void FAnimationBudgetAllocator::Update(float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimationBudgetAllocator_Update);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(AnimationBudgetAllocator);

	CheckAndPropagateUpdateStatusChange();

	if (bCurrentlyUpdating)
	{
		check(IsInGameThread());

		AllSortedComponentData.Reset();
		ReducedWorkComponentData.Reset();
		DisallowedReducedWorkComponentData.Reset();
		NonRenderedComponentData.Reset();

#if WITH_TICK_DEBUG
		AllSortedComponentDataDebug.Reset();
#endif

#if ENABLE_DRAW_DEBUG
		DisabledTickingComponentDataForDebugDraw.Reset();
		NonRenderedNotSignificantComponentDataForDebugDraw.Reset();
#endif

		QueueSortedComponentIndices(DeltaSeconds);

		float AverageTickRate = 0.0f;
		const int32 NumTicked = CalculateWorkDistributionAndQueue(DeltaSeconds, AverageTickRate);

		// Update stats			
		SET_DWORD_STAT(STAT_AnimationBudgetAllocator_NumTickedComponents, NumTicked);
		SET_DWORD_STAT(STAT_AnimationBudgetAllocator_NumRegisteredComponents, AllComponentData.Num());
		BUDGET_CSV_STAT(AnimationBudget, NumTicked, NumTicked, ECsvCustomStatOp::Set);
		BUDGET_CSV_STAT(AnimationBudget, AnimQuality, AllSortedComponentData.Num() > 0 ? (float)NumTicked / (float)AllSortedComponentData.Num() : 0.0f, ECsvCustomStatOp::Set);
		BUDGET_CSV_STAT(AnimationBudget, AverageTickRate, AverageTickRate, ECsvCustomStatOp::Set);

#if ENABLE_DRAW_DEBUG
		if(GAnimationBudgetDebugEnabled != 0)
		{
			TMap<AActor*, TArray<int32>> ActorMap;
			for (int32 ComponentDataIndex : AllSortedComponentData)
			{
				FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[ComponentDataIndex];
				TArray<int32>& ComponentIndexArray = ActorMap.FindOrAdd(ComponentData.Component->GetOwner());
				ComponentIndexArray.Add(ComponentDataIndex);
			}

			if (GAnimationBudgetDebugVerboseActive)
			{
				const float WorldTime = World->TimeSeconds - 1.0f;

				if (GAnimationBudgetDebugVerboseShowNonTicking)
				{
					for (int32 DisabledTickingComponentDataIndex : DisabledTickingComponentDataForDebugDraw)
					{
						FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[DisabledTickingComponentDataIndex];
						TArray<int32>& ComponentIndexArray = ActorMap.FindOrAdd(ComponentData.Component->GetOwner());
						ComponentIndexArray.Add(DisabledTickingComponentDataIndex);
					}

					for (int32 NonRenderedComponentDataIndex : NonRenderedNotSignificantComponentDataForDebugDraw)
					{
						FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[NonRenderedComponentDataIndex];
						TArray<int32>& ComponentIndexArray = ActorMap.FindOrAdd(ComponentData.Component->GetOwner());
						ComponentIndexArray.Add(NonRenderedComponentDataIndex);
					}
				}

				for(TPair<AActor*, TArray<int32>>& ActorIndicesPair : ActorMap)
				{
					ActorIndicesPair.Value.Sort();

					FString DebugString;
					bool bActorOffscreen = false;
				
					for(int32 ComponentDataIndex : ActorIndicesPair.Value)
					{
						FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[ComponentDataIndex];

						if (!ComponentData.bOnScreen)
						{
							bActorOffscreen = true;
						}

						// component could be not ticking due to tick being disabled externally (bTickEnabled is false) or due to being not visible, etc
						const bool bTicking = ComponentData.Component->PrimaryComponentTick.IsTickFunctionEnabled();

						if((bTicking && ComponentData.bOnScreen) || GAnimationBudgetDebugVerboseShowNonTicking)
						{
							FString StatusStr = FString::Printf(TEXT("%s %s %.2f "),
								ComponentData.Component->bHiddenInGame ? TEXT("HID") : TEXT("NOTHID"),
								ComponentData.Component->IsVisible() ? TEXT("VIS") : TEXT("NOTVIS"), 
								ComponentData.Component->GetComponentScale().X);

							if (!ComponentData.bTickEnabled)
							{
								StatusStr += TEXT(" DISABLED ");
							}
							else if (NonRenderedNotSignificantComponentDataForDebugDraw.Contains(ComponentDataIndex))
							{
								StatusStr += TEXT(" OFF (NOT SIG) ");
							}
							else if (!ComponentData.bOnScreen)
							{
								StatusStr += TEXT(" OFF (SIG) ");
							}

							FString TickStr = TEXT("   ");

							if (bTicking)
							{
								// if offscreen 
								if (ComponentData.Component->GetLastRenderTime() <= WorldTime)
								{
									switch (ComponentData.Component->VisibilityBasedAnimTickOption)
									{
										case EVisibilityBasedAnimTickOption::AlwaysTickPose:
										case EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones:
										{
											TickStr = TEXT(" A");
										}
										break;

										case EVisibilityBasedAnimTickOption::OnlyTickMontagesWhenNotRendered:
										{
											TickStr = TEXT(" M");
										}
										break;

										case EVisibilityBasedAnimTickOption::OnlyTickPoseWhenRendered:
										{
											TickStr = TEXT(" X");
										}
										break;

										default:
										break;
									}
								}
								else
								{
									TickStr = TEXT(" *");
								}
							}

							DebugString += FString::Printf(TEXT("%s%s%s%.03f %d %s %s%s - T:%.3f E:%.3f A:%.3f\n"), *StatusStr,
								(GAnimationBudgetDebugShowAddresses) ? *FString::Printf(TEXT("0x%" UPTRINT_x_FMT " "), (UPTRINT)ComponentData.Component) : TEXT(""),
								(GAnimationBudgetDebugVerboseShowNames) ? *FString::Printf(TEXT("%s%s "), (GAnimationBudgetDebugVerboseShowActorNames) ? *FString::Printf(TEXT("%s - "), *ComponentData.Component->GetOwner()->GetName()) : TEXT(""), *ComponentData.Component->GetName()) : TEXT(""),
								ComponentData.Significance, ComponentData.TickRate, ComponentData.bInterpolate ? TEXT("I") : TEXT(" "), ComponentData.bReducedWork ? TEXT("Lo") : TEXT("Hi"), *TickStr,
								ComponentData.GameThreadLastTickTimeMs, ComponentData.PrevFrameGameThreadLastCompletionTimeMs, ComponentData.GameThreadLastTickTimeMs + ComponentData.PrevFrameGameThreadLastCompletionTimeMs);
						}
					}
				
					if (GAnimationBudgetDebugVerboseOnCanvas != 0)
					{
						GEngine->AddOnScreenDebugMessage(-1, 0.0f, FColor::White, DebugString);
					}
					else
					{
						DrawDebugString(World, FVector::ZeroVector, DebugString, ActorIndicesPair.Key, FColor::White, 0.016f, false);
					}
				}

				if (bEmergencyReducedWorkActive)
				{
					GEngine->AddOnScreenDebugMessage(-1, 0.0f, FColor::Red, TEXT("Emergency Reduce Work Enabled"));
				}

				if (AllSortedComponentData.Num() > 0)
				{
					FString Prefix;
					if (World->WorldType == EWorldType::PIE && World->GetNetMode() == NM_Client)
					{
						Prefix = FString::Printf(TEXT("Client %d: "), UE::GetPlayInEditorID());
					}

					GEngine->AddOnScreenDebugMessage(-1, 0.0f, FColor::White, *FString::Printf(TEXT("%s Frame: %d Frame Budget: %.4f"), *Prefix, GFrameNumber, Parameters.BudgetInMs));
				}
			}
			else  // !GAnimationBudgetDebugVerboseActive
			{
				for (const TPair<AActor*, TArray<int32>>& ActorIndicesPair : ActorMap)
				{
					FString DebugString;

					for (int32 ComponentDataIndex : ActorIndicesPair.Value)
					{
						FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[ComponentDataIndex];
						if (ComponentData.bTickEnabled && ComponentData.bOnScreen)
						{
							if (GAnimationBudgetDebugShowAddresses != 0)
							{
								DebugString += FString::Printf(TEXT("0x%" UPTRINT_x_FMT " %d %s %s\n"), (UPTRINT)&ComponentData, ComponentData.TickRate, ComponentData.bInterpolate ? TEXT("I") : TEXT(" "), ComponentData.bReducedWork ? TEXT("Lo") : TEXT("Hi"));
							}
							else
							{
								DebugString += FString::Printf(TEXT("%.03f %d %s %s\n"), ComponentData.Significance, ComponentData.TickRate, ComponentData.bInterpolate ? TEXT("I") : TEXT(" "), ComponentData.bReducedWork ? TEXT("Lo") : TEXT("Hi"));
							}
						}
					}

					DrawDebugString(World, FVector::ZeroVector, DebugString, ActorIndicesPair.Key, FColor::White, 0.016f, false);
				}
			}

			DebugTimes.Add(FVector2D(CurrentDebugTimeDisplay, DebugTotalTime));

			float DebugTimeAccumulator = 0.0f;
			float DebugTimeNum = 0.0f;
			for(int32 DebugTimeIndex = DebugTimes.Num() - 1, DebugTimeCount = 0; DebugTimeIndex >= 0 && DebugTimeCount < 20; DebugTimeIndex--, DebugTimeCount++)
			{
				DebugTimeAccumulator += DebugTimes[DebugTimeIndex].Y;
				DebugTimeNum += 1.0f;
			}
			DebugTimesSmoothed.Add(FVector2D(CurrentDebugTimeDisplay, DebugTimeNum > 0.0f ? DebugTimeAccumulator / DebugTimeNum : 0.0f));

			CurrentDebugTimeDisplay += 0.016f;
			DebugSmoothedTotalTime = FMath::Max(FMath::CeilToFloat(DebugTimesSmoothed.Last().Y * 0.5f) / 0.5f, Parameters.BudgetInMs);
			DebugTotalTime = 0.0f;
		}
#endif
	}
}

#if ENABLE_DRAW_DEBUG
void FAnimationBudgetAllocator::OnHUDPostRender(AHUD* HUD, UCanvas* Canvas)
{
	if(FAnimationBudgetAllocator::bCachedEnabled && bEnabled && GAnimationBudgetDebugEnabled != 0 && GAnimationBudgetDebugDisableGraph == 0)
	{
		if(HUD->GetWorld() == World)
		{
			TWeakObjectPtr<UReporterGraph> Graph = Canvas->GetReporterGraph();

			Graph->SetNumGraphLines(2);

			FGraphLine* RawGraphLine = Graph->GetGraphLine(0);

			RawGraphLine->Color = FLinearColor::Gray.CopyWithNewOpacity(0.3f);
			RawGraphLine->LineName = TEXT("Raw");
			RawGraphLine->Data = DebugTimes;

			FGraphLine* SmoothedGraphLine = Graph->GetGraphLine(1);

			SmoothedGraphLine->Color = FLinearColor::White;
			SmoothedGraphLine->LineName = TEXT("Smoothed");
			SmoothedGraphLine->Data = DebugTimesSmoothed;

			Graph->SetNumThresholds(1);

			FGraphThreshold* Threshold = Graph->GetThreshold(0);
			Threshold->Color = FLinearColor::White;
			Threshold->ThresholdName = TEXT("Budget");
			Threshold->Threshold = Parameters.BudgetInMs;

			Graph->SetNotchesPerAxis(20, 3);
			Graph->SetAxesMinMax(CurrentDebugTimeDisplay - 10.0f, CurrentDebugTimeDisplay, 0.0f, DebugSmoothedTotalTime * 1.5f);
			Graph->SetGraphScreenSize(0.1f, 0.9f, 0.1f, 0.3f);
			Graph->DrawCursorOnGraph(false);
			Graph->UseTinyFont(true);
			Graph->SetCursorLocation(CurrentDebugTimeDisplay);
			Graph->SetStyles(EGraphAxisStyle::Grid, EGraphDataStyle::Lines);
			Graph->SetLegendPosition(ELegendPosition::Outside);
			Graph->OffsetDataSets(false);
			Graph->DrawExtremesOnGraph(false);
			Graph->bVisible = true;
			Graph->Draw(Canvas);

			{
				TMap<AActor*, TArray<int32>> ActorMap;
				for (int32 ComponentDataIndex = 0; ComponentDataIndex < AllComponentData.Num(); ++ComponentDataIndex)
				{
					FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[ComponentDataIndex];
					TArray<int32>& ComponentIndexArray = ActorMap.FindOrAdd(ComponentData.Component->GetOwner());
					ComponentIndexArray.Add(ComponentDataIndex);
				}

				float LineOffset = 0.0f;

				for(TPair<AActor*, TArray<int32>>& ActorIndicesPair : ActorMap)
				{
					// Sort by significance
					ActorIndicesPair.Value.Sort([this](int32 Index0, int32 Index1)
					{
						const FAnimBudgetAllocatorComponentData& ComponentData0 = AllComponentData[Index0];
						const FAnimBudgetAllocatorComponentData& ComponentData1 = AllComponentData[Index1];
 
						return ComponentData0.Significance < ComponentData1.Significance;
					});

					FVector Location = ActorIndicesPair.Key->GetActorLocation();

					FString DebugString;

					float SubLineOffset = 0.0f;

					for(int32 ComponentDataIndex : ActorIndicesPair.Value)
					{
						FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[ComponentDataIndex];
						if(ComponentData.bTickEnabled && !ComponentData.bOnScreen)
						{
							if(GAnimationBudgetDebugShowAddresses != 0)
							{
								DebugString += FString::Printf(TEXT("0x%" UPTRINT_x_FMT " %d %s %s\n"), (UPTRINT)&ComponentData, ComponentData.TickRate, ComponentData.bInterpolate ? TEXT("I") : TEXT(" "), ComponentData.bReducedWork ? TEXT("Lo") : TEXT("Hi"));
							}
							else
							{
								DebugString += FString::Printf(TEXT("%.03f %d %s %s\n"), ComponentData.Significance, ComponentData.TickRate, ComponentData.bInterpolate ? TEXT("I") : TEXT(" "), ComponentData.bReducedWork ? TEXT("Lo") : TEXT("Hi"));
							}

							SubLineOffset += 15.0f;
						}
					}

					Canvas->DrawText(GEngine->GetSmallFont(), FText::FromString(DebugString), 100.0f, 100.0f + LineOffset);

					LineOffset += SubLineOffset;
				}	
			}
		}
	}
}
#endif

void FAnimationBudgetAllocator::RemoveHelper(int32 Index, USkeletalMeshComponentBudgeted* InExpectedComponent)
{
	// Components are removed when they have been determined unreachable by GC, but may still attempt to unregister
	// themselves redundantly during teardown, so make sure the index refers to the object we expect to remove
	if (AllComponentData.IsValidIndex(Index))
	{
		USkeletalMeshComponentBudgeted* const CurrentComponent = AllComponentData[Index].Component;
		if (CurrentComponent == InExpectedComponent)
		{
			if (CurrentComponent != nullptr)
			{
				CurrentComponent->SetAnimationBudgetHandle(INDEX_NONE);
			}

			AllComponentData.RemoveAtSwap(Index, EAllowShrinking::No);

			// Update handle of swapped component
			if (AllComponentData.IsValidIndex(Index))
			{
				if (USkeletalMeshComponentBudgeted* SwappedComponent = AllComponentData[Index].Component)
				{
					SwappedComponent->SetAnimationBudgetHandle(Index);
				}
			}
		}
	}
}

static USkeletalMeshComponentBudgeted* FindRootPrerequisiteRecursive(USkeletalMeshComponentBudgeted* InComponent, TArray<USkeletalMeshComponentBudgeted*>& InVisitedComponents)
{
	InVisitedComponents.Add(InComponent);

	USkeletalMeshComponentBudgeted* Root = InComponent;

	for(FTickPrerequisite& TickPrerequisite : InComponent->PrimaryComponentTick.GetPrerequisites())
	{
		if(USkeletalMeshComponentBudgeted* PrerequisiteObject = Cast<USkeletalMeshComponentBudgeted>(TickPrerequisite.PrerequisiteObject.Get()))
		{
			if(!InVisitedComponents.Contains(PrerequisiteObject))
			{
				Root = FindRootPrerequisiteRecursive(PrerequisiteObject, InVisitedComponents);
			}
		}
	}

	return Root;
}

static USkeletalMeshComponentBudgeted* FindRootPrerequisite(USkeletalMeshComponentBudgeted* InComponent)
{
	check(IsInGameThread());

	static TArray<USkeletalMeshComponentBudgeted*> VisitedComponents;
	VisitedComponents.Reset();

	return FindRootPrerequisiteRecursive(InComponent, VisitedComponents);
}

void FAnimationBudgetAllocator::RegisterComponent(USkeletalMeshComponentBudgeted* InComponent)
{
	checkSlow(World == InComponent->GetWorld());

	const bool bIsCurrentlyEnabled = FAnimationBudgetAllocator::bCachedEnabled && bEnabled;

	if (bIsCurrentlyEnabled)
	{
		InComponent->bEnableUpdateRateOptimizations = false;
		InComponent->EnableExternalTickRateControl(true);
	}

	if (InComponent->GetAnimationBudgetHandle() == INDEX_NONE)
	{
		InComponent->SetAnimationBudgetHandle(AllComponentData.Num());

		// Setup frame offset
		FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData.Emplace_GetRef(InComponent, Parameters.InitialEstimatedWorkUnitTimeMs, Parameters.StateChangeThrottleInFrames);
		USkeletalMeshComponentBudgeted* RootPrerequisite = FindRootPrerequisite(InComponent);
		ComponentData.RootPrerequisite = (RootPrerequisite != nullptr && RootPrerequisite != InComponent) ? RootPrerequisite : nullptr;
		ComponentData.FrameOffset = CurrentFrameOffset++;
		ComponentData.bAutoCalculateSignificance = InComponent->GetAutoCalculateSignificance();

		InComponent->SetAnimationBudgetAllocator(this);
	}
	else
	{
		UpdateComponentTickPrerequsites(InComponent);
	}
}

void FAnimationBudgetAllocator::UnregisterComponent(USkeletalMeshComponentBudgeted* InComponent)
{
	checkSlow(World == InComponent->GetWorld());

	int32 ManagerHandle = InComponent->GetAnimationBudgetHandle();
	if (ManagerHandle != INDEX_NONE)
	{
		RemoveHelper(ManagerHandle, InComponent);

		InComponent->bEnableUpdateRateOptimizations = true;
		InComponent->EnableExternalTickRateControl(false);
		InComponent->SetAnimationBudgetAllocator(nullptr);

		TickEnableHelper(InComponent, true);
	}
}

void FAnimationBudgetAllocator::UnregisterAllComponents()
{
	for (int32 DataIndex = AllComponentData.Num() - 1; DataIndex >= 0; --DataIndex)
	{
		FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[DataIndex];
		if (IsValid(ComponentData.Component))
		{
			UnregisterComponent(ComponentData.Component);
		}
	}
}

void FAnimationBudgetAllocator::UpdateComponentTickPrerequsites(USkeletalMeshComponentBudgeted* InComponent)
{
	checkSlow(World == InComponent->GetWorld());

	int32 ManagerHandle = InComponent->GetAnimationBudgetHandle();
	if(ManagerHandle != INDEX_NONE)
	{
		FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[ManagerHandle];
		USkeletalMeshComponentBudgeted* RootPrerequisite = FindRootPrerequisite(InComponent);
		ComponentData.RootPrerequisite = (RootPrerequisite != nullptr && RootPrerequisite != InComponent) ? RootPrerequisite : nullptr;
	}
}

void FAnimationBudgetAllocator::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(World);

	for (FAnimBudgetAllocatorComponentData& ComponentData : AllComponentData)
	{
		Collector.AddReferencedObject(ComponentData.Component);
		Collector.AddReferencedObject(ComponentData.RootPrerequisite);
	}
}

void FAnimationBudgetAllocator::RemoveDeadComponents()
{
	for (int32 DataIndex = AllComponentData.Num() - 1; DataIndex >= 0; --DataIndex)
	{
		if (!IsValid(AllComponentData[DataIndex].Component))
		{
			RemoveHelper(DataIndex, nullptr);
		}
	}
}

void FAnimationBudgetAllocator::HandleGarbageCollectStarted()
{
	RemoveDeadComponents();
}

void FAnimationBudgetAllocator::HandlePostGarbageCollect()
{
	RemoveDeadComponents();
}

void FAnimationBudgetAllocator::SetGameThreadLastTickTimeMs(int32 InManagerHandle, float InGameThreadLastTickTimeMs)
{
	if(InManagerHandle != INDEX_NONE)
	{
		FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[InManagerHandle];
		ComponentData.GameThreadLastTickTimeMs = InGameThreadLastTickTimeMs;
#if ENABLE_DRAW_DEBUG
		DebugTotalTime += InGameThreadLastTickTimeMs;
#endif
	}
}

void FAnimationBudgetAllocator::SetGameThreadLastCompletionTimeMs(int32 InManagerHandle, float InGameThreadLastCompletionTimeMs)
{
	if(InManagerHandle != INDEX_NONE)
	{
		FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[InManagerHandle];
		ComponentData.GameThreadLastCompletionTimeMs = InGameThreadLastCompletionTimeMs;
#if ENABLE_DRAW_DEBUG
		DebugTotalTime += InGameThreadLastCompletionTimeMs;
#endif
	}
}

void FAnimationBudgetAllocator::SetIsRunningReducedWork(USkeletalMeshComponentBudgeted* InComponent, bool bInReducedWork)
{
	checkSlow(World == InComponent->GetWorld());

	int32 ManagerHandle = InComponent->GetAnimationBudgetHandle();
	if(ManagerHandle != INDEX_NONE)
	{
		FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[ManagerHandle];
		ComponentData.bReducedWork = bInReducedWork;
	}
}

void FAnimationBudgetAllocator::SetEnabled(bool bInEnabled)
{
	bEnabled = bInEnabled;
	CheckAndPropagateUpdateStatusChange();
}

bool FAnimationBudgetAllocator::GetEnabled() const
{
	return bEnabled;
}

void FAnimationBudgetAllocator::SetParameters(const FAnimationBudgetAllocatorParameters& InParameters)
{
	Parameters = InParameters;
}

void FAnimationBudgetAllocator::ForceNextTickThisFrame(USkeletalMeshComponentBudgeted* InComponent)
{
	checkSlow(World == InComponent->GetWorld());

	int32 ManagerHandle = InComponent->GetAnimationBudgetHandle();
	if (ManagerHandle != INDEX_NONE)
	{
		FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[ManagerHandle];

		// set frameoffset so next tick will be on this frame
		ComponentData.FrameOffset += (ComponentData.TickRate - ((GFrameCounter + ComponentData.FrameOffset) % ComponentData.TickRate));
	}
}

void FAnimationBudgetAllocator::AddExternalAnimationTimeSpentThisFrame(float TimeSpent)
{
	ExternalAnimationTimeSpent += TimeSpent;
}

void FAnimationBudgetAllocator::CheckAndPropagateUpdateStatusChange()
{
	FAnimationBudgetAllocator::bCachedEnabled = GAnimationBudgetEnabled == 1;
	const bool bUpdatingThisTick = FAnimationBudgetAllocator::bCachedEnabled && bEnabled;

	if (bUpdatingThisTick != bCurrentlyUpdating)
	{
		if (bUpdatingThisTick)
		{
			for (int32 DataIndex = 0; DataIndex < AllComponentData.Num(); ++DataIndex)
			{
				FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[DataIndex];
				if (ComponentData.Component != nullptr)
				{
					ComponentData.Component->bEnableUpdateRateOptimizations = false;
					ComponentData.Component->EnableExternalTickRateControl(true);
				}
			}
		}
		else
		{
			for (int32 DataIndex = 0; DataIndex < AllComponentData.Num(); ++DataIndex)
			{
				FAnimBudgetAllocatorComponentData& ComponentData = AllComponentData[DataIndex];
				if (ComponentData.Component != nullptr)
				{
					ComponentData.Component->bEnableUpdateRateOptimizations = true;
					ComponentData.Component->EnableExternalTickRateControl(false);

					// Re-enable ticking in case the component was currently skipping
					TickEnableHelper(ComponentData.Component, true);
				}
			}
		}
	}

	bCurrentlyUpdating = bUpdatingThisTick;
}

void FAnimationBudgetAllocator::SetParametersFromCVars()
{
	Parameters = GBudgetParameters;
}

void FAnimationBudgetAllocator::RegisterComponentDeferred(USkeletalMeshComponentBudgeted* InComponent)
{
	DeferredRegistrations.Add(InComponent); 
}

void FAnimationBudgetAllocator::HandleWorldBeginPlay()
{
	// This will catch worlds in (e.g.) PIE that try to set the CVar on startup
	FAnimationBudgetAllocator::bCachedEnabled = GAnimationBudgetEnabled == 1;

	bHasBegunPlay = true;

	// Run thru all deferred registrations
	for(USkeletalMeshComponentBudgeted* Component : DeferredRegistrations)
	{
		RegisterComponent(Component);
	}

	DeferredRegistrations.Empty();
}
