// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "IAnimationBudgetAllocator.h"
#include "UObject/GCObject.h"
#include "AnimationBudgetAllocatorParameters.h"
#include "DrawDebugHelpers.h"

#define UE_API ANIMATIONBUDGETALLOCATOR_API

DECLARE_STATS_GROUP(TEXT("Animation Budget Allocator"), STATGROUP_AnimationBudgetAllocator, STATCAT_Advanced);

#define WITH_TICK_DEBUG 0

// Enable this define to output more dense CSV stats about the budgeter
#define WITH_EXTRA_BUDGET_CSV_STATS		WITH_TICK_DEBUG

#if WITH_EXTRA_BUDGET_CSV_STATS
#define BUDGET_CSV_STAT(Category,StatName,Value,Op)	CSV_CUSTOM_STAT(Category,StatName,Value,Op)
#else
#define BUDGET_CSV_STAT(Category,StatName,Value,Op)	
#endif

class FAnimationBudgetAllocator;
class USkeletalMeshComponentBudgeted;
class SAnimationBudgetAllocatorDebug;
class UGameViewportClient;
class AHUD;
class UCanvas;
class FDebugDisplayInfo;

/** Data for a single component */
struct FAnimBudgetAllocatorComponentData
{
	FAnimBudgetAllocatorComponentData()
		: Component(nullptr)
		, RootPrerequisite(nullptr)
		, Significance(0.0f)
		, AccumulatedDeltaTime(0.0f)
		, GameThreadLastTickTimeMs(0.0f)
		, GameThreadLastCompletionTimeMs(0.0f)
#if ENABLE_DRAW_DEBUG
		, PrevFrameGameThreadLastCompletionTimeMs(0.0f)
#endif
		, FrameOffset(0)
		, DesiredTickRate(0)
		, TickRate(0)		
		, SkippedTicks(0)
		, StateChangeThrottle(0)
		, bTickEnabled(false)
		, bAlwaysTick(false)
		, bTickEvenIfNotRendered(false)
		, bInterpolate(false)
		, bReducedWork(false)
		, bReducedWorkDueToOffscreen(false)
		, bAllowReducedWork(false)
		, bAutoCalculateSignificance(false)
		, bOnScreen(false)
		, bWasOffScreenPrevTick(false)
		, bNeverThrottle(true)
	{}

	FAnimBudgetAllocatorComponentData(USkeletalMeshComponentBudgeted* InComponent, float InGameThreadLastTickTimeMs, int32 InStateChangeThrottle);

	bool operator==(const FAnimBudgetAllocatorComponentData& InOther) const
	{
		return Component == InOther.Component;
	}

public:
	/** The component that we are tracking */
	TObjectPtr<USkeletalMeshComponentBudgeted> Component;

	/** The root skeletal mesh component of this component's prerequisite graph, used for synchronizing ticks */
	TObjectPtr<USkeletalMeshComponentBudgeted> RootPrerequisite;

	/** Significance of this component */
	float Significance;

	/** Delta time accumulated between ticks we miss */
	float AccumulatedDeltaTime;

	/** Tracks the time in MS it took to tick this component on the game thread */
	float GameThreadLastTickTimeMs;

	/** Tracks the time in MS it took to complete this component on the game thread */
	float GameThreadLastCompletionTimeMs;

#if ENABLE_DRAW_DEBUG
	/** Keep prev frame GameThreadLastCompletionTimeMs for debug use */
	float PrevFrameGameThreadLastCompletionTimeMs;
#endif

	/** Frame offset used to distribute ticks */
	uint32 FrameOffset;

	/** The tick rate we calculated for this component */
	uint8 DesiredTickRate;

	/** The tick rate we are using for this component */
	uint8 TickRate;

	/** The current number of skipped ticks, used for determining interpolation alpha */
	uint8 SkippedTicks;

	/** Counter used to prevent state changes from happening too often */
	int8 StateChangeThrottle;

	/** Whether we ever tick */
	uint8 bTickEnabled : 1;

	/** Whether we should never skip the tick of this component, e.g. for player pawns */
	uint8 bAlwaysTick : 1;

	/** Whether we should always try to tick this component offscreen, e.g. for meshes with important audio notifies */
	uint8 bTickEvenIfNotRendered : 1;

	/** Whether we should interpolate */
	uint8 bInterpolate : 1;

	/** Whether this component is running 'reduced work' */
	uint8 bReducedWork : 1;

	/** Whether this component is running 'reduced work' due to being non-rendered offscreen */
	uint8 bReducedWorkDueToOffscreen : 1;

	/** Whether this component allows 'reduced work' */
	uint8 bAllowReducedWork : 1;

	/** Whether this component auto-calculates its significance (as opposed to it being pushed via SetComponentSignificance() */
	uint8 bAutoCalculateSignificance : 1;

	/** Whether this component is on screen. This is updated each tick. */
	uint8 bOnScreen : 1;

	/** Whether this component was off screen prev tick. */
	uint8 bWasOffScreenPrevTick : 1;

	/** Whether we are allowing interpolation on this component (i.e. we dont just reduce tick rate). This is intended to allow higher-quality animation. */
	uint8 bNeverThrottle : 1;
};

class FAnimationBudgetAllocator : public IAnimationBudgetAllocator, public FGCObject
{
public:
	UE_API FAnimationBudgetAllocator(UWorld* InWorld);
	UE_API ~FAnimationBudgetAllocator();

	// IAnimationBudgetAllocator interface
	UE_API virtual void RegisterComponent(USkeletalMeshComponentBudgeted* InComponent) override;
	UE_API virtual void UnregisterComponent(USkeletalMeshComponentBudgeted* InComponent) override;
	UE_API virtual void UnregisterAllComponents() override;
	UE_API virtual void UpdateComponentTickPrerequsites(USkeletalMeshComponentBudgeted* InComponent) override;
	UE_API virtual void SetComponentSignificance(USkeletalMeshComponentBudgeted* Component, float Significance, bool bNeverSkip = false, bool bTickEvenIfNotRendered = false, bool bAllowReducedWork = true, bool bForceInterpolate = false) override;
	UE_API virtual void SetComponentTickEnabled(USkeletalMeshComponentBudgeted* Component, bool bShouldTick) override;
	UE_API virtual bool IsComponentTickEnabled(USkeletalMeshComponentBudgeted* Component) const override;
	UE_API virtual void SetGameThreadLastTickTimeMs(int32 InManagerHandle, float InGameThreadLastTickTimeMs) override;
	UE_API virtual void SetGameThreadLastCompletionTimeMs(int32 InManagerHandle, float InGameThreadLastCompletionTimeMs) override;
	UE_API virtual void SetIsRunningReducedWork(USkeletalMeshComponentBudgeted* Component, bool bInReducedWork) override;
	UE_API virtual void Update(float DeltaSeconds) override;
	UE_API virtual void SetEnabled(bool bInEnabled) override;
	UE_API virtual bool GetEnabled() const override;
	UE_API virtual void SetParameters(const FAnimationBudgetAllocatorParameters& InParameters) override;
	UE_API virtual void ForceNextTickThisFrame(USkeletalMeshComponentBudgeted* Component) override;
	UE_API virtual void AddExternalAnimationTimeSpentThisFrame(float TimeSpent) override;

	// FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FAnimationBudgetAllocator");
	}

	// Registers a component for deferred registration (post begin play)
	UE_API void RegisterComponentDeferred(USkeletalMeshComponentBudgeted* InComponent);

	// Checks whether this allocator's world has begun play
	bool HasBegunPlay() const { return bHasBegunPlay; } 

protected:
	/** We tick before all actors in the world using this delegate */
	UE_API void OnWorldPreActorTick(UWorld* InWorld, ELevelTick InLevelTick, float InDeltaSeconds);

	/** Remove dead components backwards, readjusting indices */
	UE_API void RemoveDeadComponents();

	/** Clean up dead components at the start of GC, but before Lock. */
	UE_API void HandleGarbageCollectStarted();

	/** Clean up dead components post GC */
	UE_API void HandlePostGarbageCollect();

	/** Handle when a world begins play */
	UE_API void HandleWorldBeginPlay();
	
	/** First pass of the Update(). Queues component indices that want to tick into AllSortedComponentData. */
	UE_API void QueueSortedComponentIndices(float InDeltaSeconds);

	/**
	 * Second pass of the Update(). Looks at average time taken per unit of work (i.e. a component tick & completion task) and
	 * determines appropriate tick rates to suit the requested time budget.
	 */
	UE_API int32 CalculateWorkDistributionAndQueue(float InDeltaSeconds, float& OutAverageTickRate);

	/** Helper function for keeping handle indices in sync */
	UE_API void RemoveHelper(int32 Index, USkeletalMeshComponentBudgeted* InExpectedComponent);

	/** Helper function to enable/disable ticks */
	UE_API void TickEnableHelper(USkeletalMeshComponent* InComponent, bool bInEnable);

	/** Initializes internal parameters from CVars. */
	UE_API void SetParametersFromCVars();

	/** check for change in update status and set component external tick states to match */
	UE_API void CheckAndPropagateUpdateStatusChange();

#if ENABLE_DRAW_DEBUG
	/** Debug support function */
	UE_API void OnHUDPostRender(AHUD* HUD, UCanvas* Canvas);
#endif

protected:
	/** All of the parameters we use */
	FAnimationBudgetAllocatorParameters Parameters;

	// World we are linked to
	TObjectPtr<UWorld> World;

	// All component data
	TArray<FAnimBudgetAllocatorComponentData> AllComponentData;

	/** 
	 * All currently tickable component indices sorted by significance, updated each tick.
	 * Note that this array is not managed, so components can be deleted underneath it. 
	 * Therefore usage outside of Tick() is not recommended.
	 */
	TArray<int32> AllSortedComponentData;

#if WITH_TICK_DEBUG
	TArray<FAnimBudgetAllocatorComponentData*> AllSortedComponentDataDebug;
#endif

#if ENABLE_DRAW_DEBUG
	TArray<int32> DisabledTickingComponentDataForDebugDraw;
	TArray<int32> NonRenderedNotSignificantComponentDataForDebugDraw;
#endif

	/** All components that have reduced work that might want to tick (and hence might not want to do reduced work) */
	TArray<int32> ReducedWorkComponentData;

	/** All components that have reduced work that must now tick */
	TArray<int32> DisallowedReducedWorkComponentData;

	/** All non-rendered components we might tick */
	TArray<int32> NonRenderedComponentData;

	/** Component registrations for our world before beginplay was called */
	TArray<USkeletalMeshComponentBudgeted*> DeferredRegistrations;
	
#if ENABLE_DRAW_DEBUG
	/** Recorded debug times */
	TArray<FVector2D> DebugTimes;
	TArray<FVector2D> DebugTimesSmoothed;
#endif

	/** Average time for a work unit in milliseconds (smoothed). Updated each tick. */
	float AverageWorkUnitTimeMs;

	/** The number of components that we need to tick very frame. Updated each tick. */
	int32 NumComponentsToNotSkip;

	/** The number of components that we should not throttle (i.e. interpolate). Updated each tick. */
	int32 NumComponentsToNotThrottle;

	/** The total estimated tick time for queued ticks this frame. Updated each tick. */
	float TotalEstimatedTickTimeMs;

	/** The number of work units queued for tick this frame, used to calculate target AverageWorkUnitTimeMs. Updated each tick */
	float NumWorkUnitsForAverage;

	/** Budget pressure value, smoothed to reduce noise in 'reduced work' calculations */
	float SmoothedBudgetPressure;

#if ENABLE_DRAW_DEBUG
	/** Track total time this tick for debug display */
	float DebugTotalTime;

	/** Display time for debug graph */
	float CurrentDebugTimeDisplay;

	/** Display scale for debug graph */
	float DebugSmoothedTotalTime;
#endif

	/** Throttle counter for delaying reduced work */
	int32 ReducedComponentWorkCounter;

	/** Handle used to track start of garbage collection */
	FDelegateHandle GarbageCollectStartedHandle;

	/** Handle used to track post-garbage collection */
	FDelegateHandle PostGarbageCollectHandle;

	/** Handle used for ticking */
	FDelegateHandle OnWorldPreActorTickHandle;

	/** Handle used for CVar parameter changes */
	FDelegateHandle OnCVarParametersChangedHandle;

	/** Handle used for debug drawing */
	FDelegateHandle OnHUDPostRenderHandle;

	/** Handle used to handle world begin play */
	FDelegateHandle OnWorldBeginPlayHandle;
	
	/** Offset used to distribute component ticks */
	uint32 CurrentFrameOffset;

	/** Local enabled flag that allows us to disable even if the CVar is enabled */
	bool bEnabled;

	/** Cached enabled flag that is copied from the CVar each tick */
	static UE_API bool bCachedEnabled;

	/** Flag to indicate allocator is currently updating (ie cvar enabled and local enabled both true) */
	bool bCurrentlyUpdating;

	/** Flag to indidcate whether begin play has been called for this allocator's world */
	bool bHasBegunPlay;

	/** External animation time spent by sources not captured in ABA but that should reduce avaialbe ABA budget for the frame */
	float ExternalAnimationTimeSpent = 0.0f;

#if ENABLE_DRAW_DEBUG
	bool bEmergencyReducedWorkActive = false;
#endif
};

#undef UE_API
