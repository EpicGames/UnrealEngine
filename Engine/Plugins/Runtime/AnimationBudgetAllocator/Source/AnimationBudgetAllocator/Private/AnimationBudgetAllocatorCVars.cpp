// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationBudgetAllocatorCVars.h"
#include "AnimationBudgetAllocatorParameters.h"
#include "HAL/IConsoleManager.h"

int32 GAnimationBudgetEnabled = 0;

static FAutoConsoleVariableRef CVarAnimBudget_Enabled(
	TEXT("a.Budget.Enabled"),
	GAnimationBudgetEnabled,
	TEXT("Values: 0/1\n")
	TEXT("Controls whether the Animation Budgeting system is enabled. Should be set when there are no running skeletal meshes."),
	ECVF_Scalability);

// some optional changes to behavior in the budgeter

int32 GAnimationBudgetMaintainReduceWorkWhenOffScreen = 0;

static FAutoConsoleVariableRef CVarAnimBudget_MaintainReduceWorkWhenOffScreen(
	TEXT("a.Budget.MaintainReduceWorkWhenOffScreen"),
	GAnimationBudgetMaintainReduceWorkWhenOffScreen,
	TEXT("Values: 0/1\n")
	TEXT("Whether to enable behavior in the animation budgeter where non-rendered components keep reduce work rather than lose it when budget is available."),
	ECVF_Scalability);

int32 GAnimationBudgetComponentsWithZeroSigAreNonRendered = 0;

static FAutoConsoleVariableRef CVarAnimBudget_ComponentsWithZeroSigAreNonRendered(
	TEXT("a.Budget.ComponentsWithZeroSigAreNonRendered"),
	GAnimationBudgetComponentsWithZeroSigAreNonRendered,
	TEXT("Values: 0/1\n")
	TEXT("Control whether components with zero significance can be treated in the animation budgeter as non-rendered."),
	ECVF_Scalability);

int32 GAnimationBudgetForceTickWhenComponentExitsOffScreen = 0;

static FAutoConsoleVariableRef CVarAnimBudget_ForceTickWhenComponentExitsOffScreen(
	TEXT("a.Budget.ForceTickWhenComponentExitsOffScreen"),
	GAnimationBudgetForceTickWhenComponentExitsOffScreen,
	TEXT("Values: 0/1\n")
	TEXT("Controls whether components not ticking every frame have a forced tick when exiting offscreen."),
	ECVF_Scalability);

int32 GAnimationBudgetEmergencyReduceWorkDisabled = 0;

static FAutoConsoleVariableRef CVarAnimBudget_EmergencyReduceWorkDisabled(
	TEXT("a.Budget.EmergencyReduceWorkDisabled"),
	GAnimationBudgetEmergencyReduceWorkDisabled,
	TEXT("Values: 0/1\n")
	TEXT("Controls whether the emergency reduce work option is disabled."),
	ECVF_Scalability);

#if ENABLE_DRAW_DEBUG

int32 GAnimationBudgetDebugEnabled = 0;

static FAutoConsoleVariableRef CVarAnimBudget_DebugEnabled(
	TEXT("a.Budget.Debug.Enabled"),
	GAnimationBudgetDebugEnabled,
	TEXT("Values: 0/1\n")
	TEXT("Controls whether debug rendering (in builds that support it) is enabled for the animation budget allocator."),
	ECVF_Scalability);

int32 GAnimationBudgetDebugDisableGraph = 0;

static FAutoConsoleVariableRef CVarAnimBudget_DisableGraph(
	TEXT("a.Budget.Debug.DisableGraph"),
	GAnimationBudgetDebugDisableGraph,
	TEXT("Values: 0/1\n")
	TEXT("Controls whether debug rendering of perf graph is disabled or not."),
	ECVF_Scalability);

int32 GAnimationBudgetDebugShowAddresses = 0;

static FAutoConsoleVariableRef CVarAnimBudget_ShowAddresses(
	TEXT("a.Budget.Debug.ShowAddresses"),
	GAnimationBudgetDebugShowAddresses,
	TEXT("Values: 0/1\n")
	TEXT("Controls whether debug rendering shows addresses of component data for debugging."),
	ECVF_Scalability);

int32 GAnimationBudgetDebugVerboseActive = 0;

static FAutoConsoleVariableRef CVarAnimBudget_VerboseActive(
	TEXT("a.Budget.Debug.VerboseActive"),
	GAnimationBudgetDebugVerboseActive,
	TEXT("Values: 0/1\n")
	TEXT("Controls whether debug rendering is in verbose mode and shows a lot of extra information."),
	ECVF_Scalability);

int32 GAnimationBudgetDebugVerboseShowNames = 0;

static FAutoConsoleVariableRef CVarAnimBudget_VerboseShowNames(
	TEXT("a.Budget.Debug.Verbose.ShowNames"),
	GAnimationBudgetDebugVerboseShowNames,
	TEXT("Values: 0/1\n")
	TEXT("In Verbose Mode - Controls whether debug rendering shows names of components for debugging."),
	ECVF_Scalability);

int32 GAnimationBudgetDebugVerboseShowActorNames = 0;

static FAutoConsoleVariableRef CVarAnimBudget_VerboseShowActorNames(
	TEXT("a.Budget.Debug.Verbose.ShowActorNames"),
	GAnimationBudgetDebugVerboseShowActorNames,
	TEXT("Values: 0/1\n")
	TEXT("In Verbose Mode - Controls whether debug rendering shows actor names for debugging."),
	ECVF_Scalability);

int32 GAnimationBudgetDebugVerboseShowNonTicking = 0;

static FAutoConsoleVariableRef CVarAnimBudget_VerboseShowNonTicking(
	TEXT("a.Budget.Debug.Verbose.ShowNonTicking"),
	GAnimationBudgetDebugVerboseShowNonTicking,
	TEXT("Values: 0/1\n")
	TEXT("In Verbose Mode - Controls whether debug rendering shows non ticking component datas."),
	ECVF_Scalability);

int32 GAnimationBudgetDebugVerboseOnCanvas = 0;

static FAutoConsoleVariableRef CVarAnimBudget_VerboseShowOnCanvas(
	TEXT("a.Budget.Debug.Verbose.ShowOnCanvas"),
	GAnimationBudgetDebugVerboseOnCanvas,
	TEXT("Values: 0/1\n")
	TEXT("In Verbose Mode - Controls whether debug rendering should be shown as 2D on canvas or not."),
	ECVF_Scalability);

#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

int32 GAnimationBudgetDebugForce = 0;

static FAutoConsoleVariableRef CVarAnimBudget_Force(
	TEXT("a.Budget.Debug.Force"),
	GAnimationBudgetDebugForce,
	TEXT("Values: 0/1\n")
	TEXT("Default: 0\n")
	TEXT("Turns on forced rate/interp/reduced controls. These override any budget-driven values."),
	ECVF_Scalability);

int32 GAnimationBudgetDebugForceExcludeAlwaysTicked = 0;

static FAutoConsoleVariableRef CVarAnimBudget_ForceExcludeAlwaysTicked(
	TEXT("a.Budget.Debug.Force.ExcludeAlwaysTicked"),
	GAnimationBudgetDebugForceExcludeAlwaysTicked,
	TEXT("Values: 0/1\n")
	TEXT("Default: 0\n")
	TEXT("Excludes any components set to always tick from having forced rate/interp/reduced controls."),
	ECVF_Scalability);

int32 GAnimationBudgetDebugForceRate = 4;

static FAutoConsoleVariableRef CVarAnimBudget_ForceRate(
	TEXT("a.Budget.Debug.Force.Rate"),
	GAnimationBudgetDebugForceRate,
	TEXT("Values: >= 1\n")
	TEXT("Default: 4\n")
	TEXT("Forces all components to update at the specifed rate when a.Budget.Debug.Force is enabled."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GAnimationBudgetDebugForceRate = FMath::Max(GAnimationBudgetDebugForceRate, 1);
	}),
	ECVF_Scalability);

int32 GAnimationBudgetDebugForceInterpolation = 0;

static FAutoConsoleVariableRef CVarAnimBudget_ForceInterpolation(
	TEXT("a.Budget.Debug.Force.Interp"),
	GAnimationBudgetDebugForceInterpolation,
	TEXT("Values: 0/1\n")
	TEXT("Default: 0\n")
	TEXT("Forces interpolation on when a.Budget.Debug.Force is enabled."),
	ECVF_Scalability);

int32 GAnimationBudgetDebugForceReducedWork = 0;

static FAutoConsoleVariableRef CVarAnimBudget_ForceReducedWork(
	TEXT("a.Budget.Debug.Force.Reduced"),
	GAnimationBudgetDebugForceReducedWork,
	TEXT("Values: 0/1\n")
	TEXT("Default: 0\n")
	TEXT("Forces reduced work on when a.Budget.Debug.Force is enabled."),
	ECVF_Scalability);

#endif

/** CVar-driven parameter block */
FAnimationBudgetAllocatorParameters GBudgetParameters;

/** Delegate broadcast when parameter block changes */
FSimpleMulticastDelegate GOnCVarParametersChanged;

static FAutoConsoleVariableRef CVarAnimBudget_Budget(
	TEXT("a.Budget.BudgetMs"),
	GBudgetParameters.BudgetInMs,
	TEXT("Values > 0.1, Default = 1.0\n")
	TEXT("The time in milliseconds that we allocate for skeletal mesh work to be performed.\n")
	TEXT("When overbudget various other CVars come into play, such as a.Budget.AlwaysTickFalloffAggression and a.Budget.InterpolationFalloffAggression."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.BudgetInMs = FMath::Max(GBudgetParameters.BudgetInMs, 0.1f);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_MinQuality(
	TEXT("a.Budget.MinQuality"),
	GBudgetParameters.MinQuality,
	TEXT("Values [0.0, 1.0], Default = 0.0\n")
	TEXT("The minimum quality metric allowed. Quality is determined simply by NumComponentsTickingThisFrame / NumComponentsThatWeNeedToTick.\n")
	TEXT("If this is anything other than 0.0 then we can potentially go over our time budget."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.MinQuality = FMath::Clamp(GBudgetParameters.MinQuality, 0.0f, 1.0f);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_MaxTickRate(
	TEXT("a.Budget.MaxTickRate"),
	GBudgetParameters.MaxTickRate,
	TEXT("Values >= 1, Default = 10\n")
	TEXT("The maximum tick rate we allow. If this is set then we can potentially go over budget, but keep quality of individual meshes to a reasonable level.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.MaxTickRate = FMath::Max(GBudgetParameters.MaxTickRate, 1);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_WorkUnitSmoothingSpeed(
	TEXT("a.Budget.WorkUnitSmoothingSpeed"),
	GBudgetParameters.WorkUnitSmoothingSpeed,
	TEXT("Values > 0.1, Default = 5.0\n")
	TEXT("The speed at which the average work unit converges on the measured amount.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.WorkUnitSmoothingSpeed = FMath::Max(GBudgetParameters.WorkUnitSmoothingSpeed, 0.1f);
		GOnCVarParametersChanged.Broadcast();
	}));

static FAutoConsoleVariableRef CVarAnimBudget_AlwaysTickFalloffAggression(
	TEXT("a.Budget.AlwaysTickFalloffAggression"),
	GBudgetParameters.AlwaysTickFalloffAggression,
	TEXT("Range [0.1, 0.9], Default = 0.8\n")
	TEXT("Controls the rate at which 'always ticked' components falloff under load.\n")
	TEXT("Higher values mean that we reduce the number of always ticking components by a larger amount when the allocated time budget is exceeded."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.AlwaysTickFalloffAggression = FMath::Clamp(GBudgetParameters.AlwaysTickFalloffAggression, 0.1f, 0.9f);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_InterpolationFalloffAggression(
	TEXT("a.Budget.InterpolationFalloffAggression"),
	GBudgetParameters.InterpolationFalloffAggression,
	TEXT("Range [0.1, 0.9], Default = 0.4\n")
	TEXT("Controls the rate at which interpolated components falloff under load.\n")
	TEXT("Higher values mean that we reduce the number of interpolated components by a larger amount when the allocated time budget is exceeded.\n")
	TEXT("Components are only interpolated when the time budget is exceeded."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.InterpolationFalloffAggression = FMath::Clamp(GBudgetParameters.InterpolationFalloffAggression, 0.1f, 0.9f);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_InterpolationMaxRate(
	TEXT("a.Budget.InterpolationMaxRate"),
	GBudgetParameters.InterpolationMaxRate,
	TEXT("Values > 1, Default = 6\n")
	TEXT("Controls the rate at which ticks happen when interpolating.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.InterpolationMaxRate = FMath::Max(GBudgetParameters.InterpolationMaxRate, 2);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_MaxInterpolatedComponents(
	TEXT("a.Budget.MaxInterpolatedComponents"),
	GBudgetParameters.MaxInterpolatedComponents,
	TEXT("Range >= 0, Default = 16\n")
	TEXT("Max number of components to inteprolate before we start throttling.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.MaxInterpolatedComponents = FMath::Max(GBudgetParameters.MaxInterpolatedComponents, 0);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_InterpolationTickMultiplier(
	TEXT("a.Budget.InterpolationTickMultiplier"),
	GBudgetParameters.InterpolationTickMultiplier,
	TEXT("Range [0.1, 0.9], Default = 0.75\n")
	TEXT("Controls the expected value an amortized interpolated tick will take compared to a 'normal' tick.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.InterpolationTickMultiplier = FMath::Clamp(GBudgetParameters.InterpolationTickMultiplier, 0.1f, 0.9f);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_InitialEstimatedWorkUnitTime(
	TEXT("a.Budget.InitialEstimatedWorkUnitTime"),
	GBudgetParameters.InitialEstimatedWorkUnitTimeMs,
	TEXT("Values > 0.0, Default = 0.08\n")
	TEXT("Controls the time in milliseconds we expect, on average, for a skeletal mesh component to execute.\n")
	TEXT("The value only applies for the first tick of a component, after which we use the real time the tick takes.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.InitialEstimatedWorkUnitTimeMs = FMath::Max(GBudgetParameters.InitialEstimatedWorkUnitTimeMs, KINDA_SMALL_NUMBER);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_MaxTickedOffsreenComponents(
	TEXT("a.Budget.MaxTickedOffsreen"),
	GBudgetParameters.MaxTickedOffsreenComponents,
	TEXT("Values >= 1, Default = 4\n")
	TEXT("The maximum number of offscreen components we tick (most significant first)\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.MaxTickedOffsreenComponents = FMath::Max(GBudgetParameters.MaxTickedOffsreenComponents, 1);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_StateChangeThrottleInFrames(
	TEXT("a.Budget.StateChangeThrottleInFrames"),
	GBudgetParameters.StateChangeThrottleInFrames,
	TEXT("Range [1, 128], Default = 30\n")
	TEXT("Prevents throttle values from changing too often due to system and load noise.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.StateChangeThrottleInFrames = FMath::Clamp(GBudgetParameters.StateChangeThrottleInFrames, 1, 128);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);


static FAutoConsoleVariableRef CVarAnimBudget_BudgetFactorBeforeReducedWork(
	TEXT("a.Budget.BudgetFactorBeforeReducedWork"),
	GBudgetParameters.BudgetFactorBeforeReducedWork,
	TEXT("Range > 1, Default = 1.5\n")
	TEXT("Reduced work will be delayed until budget pressure goes over this amount.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.BudgetFactorBeforeReducedWork = FMath::Max(GBudgetParameters.BudgetFactorBeforeReducedWork, 1.0f);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_BudgetFactorBeforeReducedWorkEpsilon(
	TEXT("a.Budget.BudgetFactorBeforeReducedWorkEpsilon"),
	GBudgetParameters.BudgetFactorBeforeReducedWorkEpsilon,
	TEXT("Range > 0.0, Default = 0.25\n")
	TEXT("Increased work will be delayed until budget pressure goes under a.Budget.BudgetFactorBeforeReducedWork minus this amount.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.BudgetFactorBeforeReducedWorkEpsilon = FMath::Max(GBudgetParameters.BudgetFactorBeforeReducedWorkEpsilon, 0.0f);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_BudgetPressureSmoothingSpeed(
	TEXT("a.Budget.BudgetPressureSmoothingSpeed"),
	GBudgetParameters.BudgetPressureSmoothingSpeed,
	TEXT("Range > 0.0, Default = 3.0\n")
	TEXT("How much to smooth the budget pressure value used to throttle reduced work.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.BudgetPressureSmoothingSpeed = FMath::Max(GBudgetParameters.BudgetPressureSmoothingSpeed, KINDA_SMALL_NUMBER);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_ReducedWorkThrottleMinInFrames(
	TEXT("a.Budget.ReducedWorkThrottleMinInFrames"),
	GBudgetParameters.ReducedWorkThrottleMinInFrames,
	TEXT("Range [1, 255], Default = 2\n")
	TEXT("Prevents reduced work from changing too often due to system and load noise. Min value used when over budget pressure (i.e. aggressive reduction).\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.ReducedWorkThrottleMinInFrames = FMath::Clamp(GBudgetParameters.ReducedWorkThrottleMinInFrames, 1, 255);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_ReducedWorkThrottleMaxInFrames(
	TEXT("a.Budget.ReducedWorkThrottleMaxInFrames"),
	GBudgetParameters.ReducedWorkThrottleMaxInFrames,
	TEXT("Range [1, 255], Default = 20\n")
	TEXT("Prevents reduced work from changing too often due to system and load noise. Max value used when under budget pressure.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.ReducedWorkThrottleMaxInFrames = FMath::Clamp(GBudgetParameters.ReducedWorkThrottleMaxInFrames, 1, 255);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_BudgetFactorBeforeAggressiveReducedWork(
	TEXT("a.Budget.BudgetFactorBeforeAggressiveReducedWork"),
	GBudgetParameters.BudgetFactorBeforeAggressiveReducedWork,
	TEXT("Range > 1, Default = 2.0\n")
	TEXT("Reduced work will be applied more rapidly when budget pressure goes over this amount.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.BudgetFactorBeforeAggressiveReducedWork = FMath::Max(GBudgetParameters.BudgetFactorBeforeAggressiveReducedWork, 1.0f);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_ReducedWorkThrottleMaxPerFrame(
	TEXT("a.Budget.ReducedWorkThrottleMaxPerFrame"),
	GBudgetParameters.ReducedWorkThrottleMaxPerFrame,
	TEXT("Range [1, 255], Default = 4\n")
	TEXT("Controls the max number of components that are switched to/from reduced work per tick.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.ReducedWorkThrottleMaxPerFrame = FMath::Clamp(GBudgetParameters.ReducedWorkThrottleMaxPerFrame, 1, 255);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_BudgetPressureBeforeEmergencyReducedWork(
	TEXT("a.Budget.BudgetPressureBeforeEmergencyReducedWork"),
	GBudgetParameters.BudgetPressureBeforeEmergencyReducedWork,
	TEXT("Range > 0.0, Default = 2.5\n")
	TEXT("Controls the budget pressure where emergency reduced work (applied to all components except those that are bAlwaysTick).\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.BudgetPressureBeforeEmergencyReducedWork = FMath::Max(GBudgetParameters.BudgetPressureBeforeEmergencyReducedWork, 0.0f);
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);

static FAutoConsoleVariableRef CVarAnimBudget_AutoCalculatedSignificanceMaxDistance(
	TEXT("a.Budget.AutoCalculatedSignificanceMaxDistance"),
	GBudgetParameters.AutoCalculatedSignificanceMaxDistance,
	TEXT("Range > 1.0, Default = 300.0\n")
	TEXT("Controls the distance (in cm) at which auto-calculated significance for budgeted components bottoms out. Components within the distance 1 -> Max will have significance mapped 1 -> 0, outside of MaxDistance significance will be zero.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		GBudgetParameters.AutoCalculatedSignificanceMaxDistance = FMath::Max(GBudgetParameters.AutoCalculatedSignificanceMaxDistance, 1.0f);
		GBudgetParameters.AutoCalculatedSignificanceMaxDistanceSqr = GBudgetParameters.AutoCalculatedSignificanceMaxDistance * GBudgetParameters.AutoCalculatedSignificanceMaxDistance;
		GOnCVarParametersChanged.Broadcast();
	}),
	ECVF_Scalability);