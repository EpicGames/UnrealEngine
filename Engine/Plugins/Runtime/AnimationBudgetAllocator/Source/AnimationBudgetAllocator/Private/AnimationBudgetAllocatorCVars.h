// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DrawDebugHelpers.h"

struct FAnimationBudgetAllocatorParameters;

/** Enabled/disabled flag */
extern int32 GAnimationBudgetEnabled;

extern int32 GAnimationBudgetMaintainReduceWorkWhenOffScreen;

extern int32 GAnimationBudgetComponentsWithZeroSigAreNonRendered;

extern int32 GAnimationBudgetForceTickWhenComponentExitsOffScreen;

/* Controls whether the emergency reduce work option is disabled. */
extern int32 GAnimationBudgetEmergencyReduceWorkDisabled;

#if ENABLE_DRAW_DEBUG

/** Debug rendering flag */
extern int32 GAnimationBudgetDebugEnabled;

/* Controls whether debug rendering of perf graph is disabled or not. */
extern int32 GAnimationBudgetDebugDisableGraph;

/** Controls whether debug rendering shows addresses of component data for debugging */
extern int32 GAnimationBudgetDebugShowAddresses;

/** Controls whether debug rendering is in verbose mode and shows a lot of extra information */
extern int32 GAnimationBudgetDebugVerboseActive;

/* In Verbose Mode - Controls whether debug rendering shows names of components for debugging. */
extern int32 GAnimationBudgetDebugVerboseShowNames;

/* In Verbose Mode - Controls whether debug rendering shows actor names for debugging. */
extern int32 GAnimationBudgetDebugVerboseShowActorNames;

/* In Verbose Mode - Controls whether debug rendering shows non ticking component datas. */
extern int32 GAnimationBudgetDebugVerboseShowNonTicking;

/* In Verbose Mode - Controls whether debug rendering should be shown as 2D on canvas or not. */
extern int32 GAnimationBudgetDebugVerboseOnCanvas;

#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

/** Parameters used to force state for debugging */
extern int32 GAnimationBudgetDebugForce;
extern int32 GAnimationBudgetDebugForceExcludeAlwaysTicked;
extern int32 GAnimationBudgetDebugForceRate;
extern int32 GAnimationBudgetDebugForceInterpolation;
extern int32 GAnimationBudgetDebugForceReducedWork;

#endif

/** CVar-driven parameter block */
extern FAnimationBudgetAllocatorParameters GBudgetParameters;

/** Delegate broadcast when parameter block changes */
extern FSimpleMulticastDelegate GOnCVarParametersChanged;
