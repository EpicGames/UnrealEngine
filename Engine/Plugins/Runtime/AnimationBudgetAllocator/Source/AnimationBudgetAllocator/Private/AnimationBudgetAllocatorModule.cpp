// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationBudgetAllocatorModule.h"
#include "AnimationBudgetAllocator.h"

IMPLEMENT_MODULE(FAnimationBudgetAllocatorModule, AnimationBudgetAllocator);

IAnimationBudgetAllocator* FAnimationBudgetAllocatorModule::GetBudgetAllocatorForWorld(UWorld* World)
{
	check(World);

	FAnimationBudgetAllocator* Budgeter = WorldAnimationBudgetAllocators.FindRef(World);
	if(Budgeter == nullptr && World->IsGameWorld())
	{
		Budgeter = new FAnimationBudgetAllocator(World);
		WorldAnimationBudgetAllocators.Add(World, Budgeter);
	}

	return Budgeter;
}

void FAnimationBudgetAllocatorModule::StartupModule()
{
	PreWorldInitializationHandle = FWorldDelegates::OnPreWorldInitialization.AddRaw(this, &FAnimationBudgetAllocatorModule::HandleWorldInit);
	PostWorldCleanupHandle = FWorldDelegates::OnPostWorldCleanup.AddRaw(this, &FAnimationBudgetAllocatorModule::HandleWorldCleanup);
}

void FAnimationBudgetAllocatorModule::ShutdownModule()
{
	FWorldDelegates::OnPreWorldInitialization.Remove(PreWorldInitializationHandle);
	FWorldDelegates::OnPostWorldCleanup.Remove(PostWorldCleanupHandle);
}

void FAnimationBudgetAllocatorModule::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& WorldAnimationBudgetAllocatorPair : WorldAnimationBudgetAllocators)
	{
		Collector.AddReferencedObject(WorldAnimationBudgetAllocatorPair.Key);
	}
}

void FAnimationBudgetAllocatorModule::HandleWorldInit(UWorld* World, const UWorld::InitializationValues IVS)
{
	GetBudgetAllocatorForWorld(World);
}

void FAnimationBudgetAllocatorModule::HandleWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	FAnimationBudgetAllocator* Budgeter = WorldAnimationBudgetAllocators.FindRef(World);
	if(Budgeter)
	{
		//Cleanup components, as this can cause a crash in some conditions during seamless travel as actors can be pulled in
		//to the travel world.
		Budgeter->SetEnabled(false);
		Budgeter->UnregisterAllComponents();
		delete Budgeter;
		WorldAnimationBudgetAllocators.Remove(World);
	}
}
