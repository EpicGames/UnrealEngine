// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "IAnimationBudgetAllocatorModule.h"
#include "Engine/World.h"

#define UE_API ANIMATIONBUDGETALLOCATOR_API

class FAnimationBudgetAllocator;

class FAnimationBudgetAllocatorModule : public IAnimationBudgetAllocatorModule, public FGCObject
{
public:
	// IAnimationBudgetAllocatorModule interface
	UE_API virtual IAnimationBudgetAllocator* GetBudgetAllocatorForWorld(UWorld* World) override;

	// IModuleInterface interface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	// FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FAnimationBudgetAllocatorModule");
	}

private:
	/** Handle world initialization */
	UE_API void HandleWorldInit(UWorld* World, const UWorld::InitializationValues IVS);

	/** Handle world cleanup */
	UE_API void HandleWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

	/** Delegate handles for hooking into UWorld lifetimes */
	FDelegateHandle PreWorldInitializationHandle;
	FDelegateHandle PostWorldCleanupHandle;

	/** Map of world->budgeter */
	TMap<TObjectPtr<UWorld>, FAnimationBudgetAllocator*> WorldAnimationBudgetAllocators;
};

#undef UE_API
