// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationBudgetBlueprintLibrary.h"
#include "AnimationBudgetAllocatorModule.h"
#include "IAnimationBudgetAllocator.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationBudgetBlueprintLibrary)

void UAnimationBudgetBlueprintLibrary::EnableAnimationBudget(UObject* WorldContextObject, bool bEnabled)
{
	if(UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FAnimationBudgetAllocatorModule& AnimationBudgetAllocatorModule = FModuleManager::LoadModuleChecked<FAnimationBudgetAllocatorModule>("AnimationBudgetAllocator");
		IAnimationBudgetAllocator* AnimationBudgetAllocator = AnimationBudgetAllocatorModule.GetBudgetAllocatorForWorld(World);
		if(AnimationBudgetAllocator)
		{
			AnimationBudgetAllocator->SetEnabled(bEnabled);
		}
	}
}

void UAnimationBudgetBlueprintLibrary::SetAnimationBudgetParameters(UObject* WorldContextObject, const FAnimationBudgetAllocatorParameters& InParameters)
{
	if(UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FAnimationBudgetAllocatorModule& AnimationBudgetAllocatorModule = FModuleManager::LoadModuleChecked<FAnimationBudgetAllocatorModule>("AnimationBudgetAllocator");
		IAnimationBudgetAllocator* AnimationBudgetAllocator = AnimationBudgetAllocatorModule.GetBudgetAllocatorForWorld(World);
		if(AnimationBudgetAllocator)
		{
			AnimationBudgetAllocator->SetParameters(InParameters);
		}
	}
}
