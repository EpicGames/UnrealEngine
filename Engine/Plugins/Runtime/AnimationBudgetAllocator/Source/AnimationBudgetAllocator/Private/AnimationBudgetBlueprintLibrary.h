// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "AnimationBudgetBlueprintLibrary.generated.h"

struct FAnimationBudgetAllocatorParameters;

/**
 * Function library to expose the budget allocator to Blueprints
 */
UCLASS(meta = (ScriptName = "AnimationBudget"))
class UAnimationBudgetBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	/**
	 * Enable/disable the animation budgeting system.
	 * Note that the system can also be disabled globally via CVar, which overrides this setting.
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation Budget", meta=(WorldContext="WorldContextObject"))
	static void EnableAnimationBudget(UObject* WorldContextObject, bool bEnabled);

	/**
	 * Set the parameters of the animation budgeting system.
	 * Note that the parameters can also be set globally via CVar.
	 */
	UFUNCTION(BlueprintCallable, Category = "Animation Budget", meta=(WorldContext="WorldContextObject"))
	static void SetAnimationBudgetParameters(UObject* WorldContextObject, const FAnimationBudgetAllocatorParameters& InParameters);
};
