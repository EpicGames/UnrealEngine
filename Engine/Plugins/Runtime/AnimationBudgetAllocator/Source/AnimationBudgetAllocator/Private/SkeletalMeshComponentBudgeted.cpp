// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshComponentBudgeted.h"
#include "AnimationBudgetAllocator.h"
#include "AnimationBudgetAllocatorCVars.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Engine/World.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "GameFramework/Controller.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshComponentBudgeted)

CSV_DECLARE_CATEGORY_EXTERN(AnimationBudget);

#if UE_BUILD_SHIPPING && !WITH_EDITOR
static_assert(sizeof(USkeletalMeshComponentBudgeted) <= 4368, "We expect SkeletalMeshComponentBudgeted to be 4368 or less, otherwise we waste memory on 304bytes of padding on MallocBinned2. Please improve padding on this class hierarchy to pay for your added variable.");
#endif

FOnCalculateSignificance USkeletalMeshComponentBudgeted::OnCalculateSignificanceDelegate;

USkeletalMeshComponentBudgeted::USkeletalMeshComponentBudgeted(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAutoRegisterWithBudgetAllocator(true)
	, bAutoCalculateSignificance(false)
	, bShouldUseActorRenderedFlag(false)
{
}

void USkeletalMeshComponentBudgeted::BeginPlay()
{
	Super::BeginPlay();

	if(bAutoRegisterWithBudgetAllocator && !UKismetSystemLibrary::IsDedicatedServer(this))
	{
		if (UWorld* LocalWorld = GetWorld())
		{
			if (FAnimationBudgetAllocator* LocalAnimationBudgetAllocator = static_cast<FAnimationBudgetAllocator*>(IAnimationBudgetAllocator::Get(LocalWorld)))
			{
				if(LocalAnimationBudgetAllocator->HasBegunPlay())
				{
					// World is playing, so register 
					LocalAnimationBudgetAllocator->RegisterComponent(this);
				}
				else
				{
					// World is not playing, so register deferred 
					LocalAnimationBudgetAllocator->RegisterComponentDeferred(this);
				}
			}
		}
	}
}

void USkeletalMeshComponentBudgeted::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (UWorld* LocalWorld = GetWorld())
	{
		if (IAnimationBudgetAllocator* LocalAnimationBudgetAllocator = IAnimationBudgetAllocator::Get(LocalWorld))
		{
			LocalAnimationBudgetAllocator->UnregisterComponent(this);
		}
	}

	Super::EndPlay(EndPlayReason);
}

void USkeletalMeshComponentBudgeted::SetComponentTickEnabled(bool bEnabled)
{
	if (AnimationBudgetAllocator)
	{
		AnimationBudgetAllocator->SetComponentTickEnabled(this, bEnabled);
	}
	else
	{
		Super::SetComponentTickEnabled(bEnabled);
	}
}

void USkeletalMeshComponentBudgeted::SetComponentSignificance(float Significance, bool bNeverSkip, bool bTickEvenIfNotRendered, bool bAllowReducedWork, bool bForceInterpolate)
{
	if (AnimationBudgetAllocator)
	{
		AnimationBudgetAllocator->SetComponentSignificance(this, Significance, bNeverSkip, bTickEvenIfNotRendered, bAllowReducedWork, bForceInterpolate);
	}
	else if (HasBegunPlay())
	{
		UE_LOGF(LogSkeletalMesh, Warning, "SetComponentSignificance called on [%ls] before registering with budget allocator", *GetName());
	}
}

void USkeletalMeshComponentBudgeted::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
#if !UE_BUILD_SHIPPING
	CSV_SCOPED_TIMING_STAT(AnimationBudget, BudgetedAnimation);
#endif

	if(AnimationBudgetAllocator)
	{
		uint64 StartTime = FPlatformTime::Cycles64();

		Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

		if(bAutoCalculateSignificance && !USkeletalMeshComponentBudgeted::OnCalculateSignificance().IsBound())
		{
			// Default auto-calculated significance is based on distance to each player viewpoint
			AutoCalculatedSignificance = 0.0f;

			const FVector WorldLocation = GetComponentTransform().GetLocation();
			const UWorld* ComponentWorld = GetWorld();
			for (FConstPlayerControllerIterator It = ComponentWorld->GetPlayerControllerIterator(); It; ++It)
			{
				if(const APlayerController* PlayerController = It->Get())
				{
					FVector PlayerViewLocation = FVector::ZeroVector;
					FRotator PlayerViewRotation = FRotator::ZeroRotator;
					PlayerController->GetPlayerViewPoint(PlayerViewLocation, PlayerViewRotation);

					const float DistanceSqr = (WorldLocation - PlayerViewLocation).SizeSquared();
					const float Significance = FMath::Max(GBudgetParameters.AutoCalculatedSignificanceMaxDistanceSqr - DistanceSqr, 1.0f) / GBudgetParameters.AutoCalculatedSignificanceMaxDistanceSqr;
					AutoCalculatedSignificance = FMath::Max(AutoCalculatedSignificance, Significance);
				}
			}
		}
		
		if(AnimationBudgetAllocator)
		{
			AnimationBudgetAllocator->SetGameThreadLastTickTimeMs(AnimationBudgetHandle, FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartTime));
		}
	}
	else
	{
		Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	}
}

void USkeletalMeshComponentBudgeted::CompleteParallelAnimationEvaluation(bool bDoPostAnimEvaluation)
{
#if !UE_BUILD_SHIPPING
	CSV_SCOPED_TIMING_STAT(AnimationBudget, BudgetedAnimation);
#endif

	if(AnimationBudgetAllocator)
	{
		uint64 StartTime = FPlatformTime::Cycles64();

		Super::CompleteParallelAnimationEvaluation(bDoPostAnimEvaluation);

		if(AnimationBudgetAllocator)
		{
			AnimationBudgetAllocator->SetGameThreadLastCompletionTimeMs(AnimationBudgetHandle, FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - StartTime));
		}
	}
	else
	{
		Super::CompleteParallelAnimationEvaluation(bDoPostAnimEvaluation);
	}
}

float USkeletalMeshComponentBudgeted::GetDefaultSignificance() const
{
	return AutoCalculatedSignificance;
}
