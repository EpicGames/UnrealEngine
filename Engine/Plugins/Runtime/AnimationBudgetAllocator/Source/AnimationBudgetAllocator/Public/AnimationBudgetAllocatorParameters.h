// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationBudgetAllocatorParameters.generated.h"

/** Parameter block used to control the behavior of the budget allocator */
USTRUCT(BlueprintType)
struct FAnimationBudgetAllocatorParameters
{
	GENERATED_BODY()

	/**
	 * Values > 0.1.
	 * The time in milliseconds that we allocate for skeletal mesh work to be performed.
	 * When overbudget various other parameters come into play, such as AlwaysTickFalloffAggression and InterpolationFalloffAggression.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float BudgetInMs = 1.0f;

	/**
	 * Values [0.0, 1.0].
	 * The minimum quality metric allowed. Quality is determined simply by NumComponentsTickingThisFrame / NumComponentsThatWeNeedToTick.
	 * If this is anything other than 0.0 then we can potentially go over our time budget.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float MinQuality = 0.0f;

	/** 
	 * Values >= 1.
	 * The maximum tick rate we allow. If this is set then we can potentially go over budget, but keep quality of individual meshes to a reasonable level. 
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	int32 MaxTickRate = 10;

	/** 
	 * Values > 0.1.
	 * The speed at which the average work unit converges on the measured amount. 
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float WorkUnitSmoothingSpeed = 5.0f;

	/**
	 * Range [0.1, 0.9].
	 * Controls the rate at which 'always ticked' components falloff under load.
	 * Higher values mean that we reduce the number of always ticking components by a larger amount when the allocated time budget is exceeded.
	*/
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float AlwaysTickFalloffAggression = 0.8f;

	/**
	 * Range [0.1, 0.9].
	 * Controls the rate at which interpolated components falloff under load.
	 * Higher values mean that we reduce the number of interpolated components by a larger amount when the allocated time budget is exceeded.
	 * Components are only interpolated when the time budget is exceeded.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float InterpolationFalloffAggression = 0.4f;

	/** 
	 * Values > 1.
	 * Controls the rate at which ticks happen when interpolating. 
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	int32 InterpolationMaxRate = 6;

	/** 
	 * Range >= 0.
	 * Max number of components to interpolate before we start throttling. 
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	int32 MaxInterpolatedComponents = 16;

	/** 
	 * Range [0.1, 0.9].
	 * Controls the expected value an amortized interpolated tick will take compared to a 'normal' tick. 
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float InterpolationTickMultiplier = 0.75f;

	/** 
	 * Values > 0.0.
	 * Controls the time in milliseconds we expect, on average, for a skeletal mesh component to execute.
	 * The value only applies for the first tick of a component, after which we use the real time the tick takes. 
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float InitialEstimatedWorkUnitTimeMs = 0.08f;

	/**
	 * Values >= 1
	 * The maximum number of offscreen components we tick (most significant first)
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	int32 MaxTickedOffsreenComponents = 4;

	/**
	 * Range [1, 128]
	 * Prevents throttle values from changing too often due to system and load noise.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	int32 StateChangeThrottleInFrames = 30;

	/**
	 * Range > 1
	 * Reduced work will be delayed until budget pressure goes over this amount.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float BudgetFactorBeforeReducedWork = 1.5f;

	/** 
	 * Range > 0.0.
	 * Increased work will be delayed until budget pressure goes under BudgetFactorBeforeReducedWork minus this amount. 
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float BudgetFactorBeforeReducedWorkEpsilon = 0.25f;

	/**
	 * Range > 0.0.
	 * How much to smooth the budget pressure value used to throttle reduced work.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float BudgetPressureSmoothingSpeed = 3.0f;

	/**
	 * Range [1, 255].
	 * Prevents reduced work from changing too often due to system and load noise. Min value used when over budget pressure (i.e. aggressive reduction).
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	int32 ReducedWorkThrottleMinInFrames = 2;

	/**
	 * Range [1, 255].
	 * Prevents reduced work from changing too often due to system and load noise. Max value used when under budget pressure.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	int32 ReducedWorkThrottleMaxInFrames = 20;

	/**
	 * Range > 1.
	 * Reduced work will be applied more rapidly when budget pressure goes over this amount.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float BudgetFactorBeforeAggressiveReducedWork = 2.0f;

	/**
	 * Range [1, 255].
	 * Controls the max number of components that are switched to/from reduced work per tick.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	int32 ReducedWorkThrottleMaxPerFrame = 4;

	/**
	 * Range > 0.0.
	 * Controls the budget pressure where emergency reduced work (applied to all components except those that are bAlwaysTick).
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float BudgetPressureBeforeEmergencyReducedWork = 2.5f;

	/**
	 * Range > 1.0.
	 * Controls the distance at which auto-calculated significance for budgeted components bottoms out. Components
	 * within the distance 1 -> Max will have significance mapped 1 -> 0, outside of MaxDistance significance will be zero.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	float AutoCalculatedSignificanceMaxDistance = 30000.0f;
	float AutoCalculatedSignificanceMaxDistanceSqr = 30000.0f * 30000.0f;
};