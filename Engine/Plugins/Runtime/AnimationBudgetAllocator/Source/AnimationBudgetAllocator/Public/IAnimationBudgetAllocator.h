// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class USkeletalMeshComponentBudgeted;
class UWorld;
struct FAnimationBudgetAllocatorParameters;

/**
 * Dynamically manages skeletal mesh component tick rates to try to maintain a specified budget.
 */
class IAnimationBudgetAllocator
{
public:
	/** Get the budgeter for the specified world */
	static ANIMATIONBUDGETALLOCATOR_API IAnimationBudgetAllocator* Get(UWorld* InWorld);

	/**
	 * Register a component with the budgeter system. If the component is already registered this function does nothing.
	 * Once this is called:
	 * - Default tick function will be disabled
	 * - URO will be disabled
	 * - Parallel anim tasks will be re-routed to the budgeter
	 */
	virtual void RegisterComponent(USkeletalMeshComponentBudgeted* InComponent) = 0;

	/**
	 * Unregister a component from the budgeter system. If the component is not registered this function does nothing.
	 * Once this is called:
	 * - Default tick function will be re-enabled
	 * - URO will be re-enabled
	 * - Parallel anim tasks will be re-routed back to internal functions
	 */
	virtual void UnregisterComponent(USkeletalMeshComponentBudgeted* InComponent) = 0;

	/**
	 * Calls UnregisterComponent on all components currently registered with the allocator.
	 */
	virtual void UnregisterAllComponents() = 0;

	/**
	 * Update the prerequisites of this component. Should be called when prerequisites may have changed externally.
	 */
	virtual void UpdateComponentTickPrerequsites(USkeletalMeshComponentBudgeted* InComponent) = 0;

	/**
	 * Set the significance and other flags for the specified component.
	 * This information is used to dynamically control the tick rate of the component.
	 */
	virtual void SetComponentSignificance(USkeletalMeshComponentBudgeted* Component, float Significance, bool bNeverSkip = false, bool bTickEvenIfNotRendered = false, bool bAllowReducedWork = true, bool bForceInterpolate = false) = 0;

	/** Set the specified component to tick or not. If the budgeter is disabled then this calls Component->SetComponentTickEnabled(bShouldTick). */
	virtual void SetComponentTickEnabled(USkeletalMeshComponentBudgeted* Component, bool bShouldTick) = 0;

	/** Get whether the specified component is set to tick or not */
	virtual bool IsComponentTickEnabled(USkeletalMeshComponentBudgeted* Component) const = 0;

	/** Inform that we reduced work for a component */
	virtual void SetIsRunningReducedWork(USkeletalMeshComponentBudgeted* Component, bool bInReducedWork) = 0;

	/** Set the tick time */
	virtual void SetGameThreadLastTickTimeMs(int32 InManagerHandle, float InGameThreadLastTickTimeMs) = 0;

	/** Set the completion task time */
	virtual void SetGameThreadLastCompletionTimeMs(int32 InManagerHandle, float InGameThreadLastCompletionTimeMs) = 0;

	/** Tick the system per-frame */
	virtual void Update(float DeltaSeconds) = 0;

	/** Set whether this budget allocator is enabled */
	virtual void SetEnabled(bool bInEnabled) = 0;

	/** Get whether this budget allocator is enabled */
	virtual bool GetEnabled() const = 0;

	/** Set the various parameters */
	virtual void SetParameters(const FAnimationBudgetAllocatorParameters& InParameters) = 0;

	/** Force component to tick this frame regardless of tick rate */
	virtual void ForceNextTickThisFrame(USkeletalMeshComponentBudgeted* Component) = 0;

	/**
	* Add external time spent on animation that should come out of the Animation Budget but is not accounted for in the ABA update.
	* TimeSpent is accumulated and spent each frame, it is reset to 0 each time it is used.  
	*/
	virtual void AddExternalAnimationTimeSpentThisFrame(float TimeSpent) = 0;
};