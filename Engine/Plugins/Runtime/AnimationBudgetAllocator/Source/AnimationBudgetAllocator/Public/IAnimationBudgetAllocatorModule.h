// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class IModuleInterface;

class IAnimationBudgetAllocator;

class IAnimationBudgetAllocatorModule : public FDefaultGameModuleImpl
{
public:
	/**
	 * Get a budgeter for a specific world. Adds it if it is not already created.
	 * Budgeters are only created for game worlds.
	 */
	virtual IAnimationBudgetAllocator* GetBudgetAllocatorForWorld(UWorld* World) = 0;

	/** Get a budgeter for a specific world. Fairly expensive as it calls into module manager and performs a map lookup. */
	inline static IAnimationBudgetAllocator* Get(UWorld* World)
	{
		IAnimationBudgetAllocatorModule& AnimationBudgetAllocatorModule = FModuleManager::LoadModuleChecked<IAnimationBudgetAllocatorModule>("AnimationBudgetAllocator");
		return AnimationBudgetAllocatorModule.GetBudgetAllocatorForWorld(World);
	}
};
