// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SkeletalMeshComponent.h"
#include "SkeletalMeshComponentBudgeted.generated.h"

#define UE_API ANIMATIONBUDGETALLOCATOR_API

namespace EEndPlayReason { enum Type : int; }

class IAnimationBudgetAllocator;
class USkeletalMeshComponentBudgeted;

/** Delegate called to increase/decrease the amount of work a component performs */
DECLARE_DELEGATE_TwoParams(FOnReduceWork, USkeletalMeshComponentBudgeted* /*InComponent*/, bool /*bReduce*/);

/** Delegate called to calculate significance if bAutoCalculateSignificance = true */
DECLARE_DELEGATE_RetVal_OneParam(float, FOnCalculateSignificance, USkeletalMeshComponentBudgeted* /*InComponent*/);

/** A skeletal mesh component that has its tick rate governed by a global animation budget */
UCLASS(MinimalAPI, meta=(BlueprintSpawnableComponent))
class USkeletalMeshComponentBudgeted : public USkeletalMeshComponent
{
	GENERATED_BODY()

	friend class FAnimationBudgetAllocator;

public:
	UE_API USkeletalMeshComponentBudgeted(const FObjectInitializer& ObjectInitializer);

	// UActorComponent interface
	UE_API virtual void SetComponentTickEnabled(bool bEnabled) override;

	/** Updates significance budget if this component has been registered with a AnimationBudgetAllocator */
	UE_API void SetComponentSignificance(float Significance, bool bNeverSkip = false, bool bTickEvenIfNotRendered = false, bool bAllowReducedWork = true, bool bForceInterpolate = false);

	/** Set this component to automatically register with the budget allocator */
	UFUNCTION(BlueprintSetter)
	void SetAutoRegisterWithBudgetAllocator(bool bInAutoRegisterWithBudgetAllocator) { bAutoRegisterWithBudgetAllocator = bInAutoRegisterWithBudgetAllocator; }

	/** Set this component to automatically calculate its significance */
	void SetAutoCalculateSignificance(bool bInAutoCalculateSignificance) { bAutoCalculateSignificance = bInAutoCalculateSignificance; }

	/** Check whether this component auto-calculates its significance */
	bool GetAutoCalculateSignificance() const { return bAutoCalculateSignificance; }

	/** Get delegate called to increase/decrease the amount of work a component performs */
	FOnReduceWork& OnReduceWork() { return OnReduceWorkDelegate; }

	/** Get delegate called to calculate significance if bAutoCalculateSignificance = true */
	static FOnCalculateSignificance& OnCalculateSignificance() { return OnCalculateSignificanceDelegate; }

	bool GetShouldUseActorRenderedFlag() const { return bShouldUseActorRenderedFlag; };

	void SetShouldUseActorRenderedFlag(bool value) { bShouldUseActorRenderedFlag = value; };

protected:

	// USkeletalMeshComponent interface
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	UE_API virtual void CompleteParallelAnimationEvaluation(bool bDoPostAnimEvaluation) override;

	// UActorComponent interface
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

private:

	/** Get the handle used to identify this component to the allocator */
	int32 GetAnimationBudgetHandle() const { return AnimationBudgetHandle; }

	/** Set the handle used to identify this component to the allocator */
	void SetAnimationBudgetHandle(int32 InHandle) { AnimationBudgetHandle = InHandle; }

	/** Set the budget allocator that is tracking us */
	void SetAnimationBudgetAllocator(IAnimationBudgetAllocator* InAnimationBudgetAllocator) { AnimationBudgetAllocator = InAnimationBudgetAllocator; }

	/** Handles self-registration after actors have begun play */
	UE_API void HandleWorldBeginPlay();

	/** Gets the default significance value based on distance */
	UE_API float GetDefaultSignificance() const;
	
private:
	/** Delegate called to increase/decrease the amount of work a component performs */
	FOnReduceWork OnReduceWorkDelegate;

	/** Delegate called to calculate significance if bAutoCalculateSignificance = true */
	static UE_API FOnCalculateSignificance OnCalculateSignificanceDelegate;
	
	/** Owning animation budget allocator */
	IAnimationBudgetAllocator* AnimationBudgetAllocator = nullptr;

	/** Handle used for identification */
	int32 AnimationBudgetHandle = INDEX_NONE;

	/** Cached calculated significance */
	float AutoCalculatedSignificance = 1.0f;
	
	/** Whether this component should automatically register with the budget allocator in OnRegister/OnUnregister */
	UPROPERTY(EditAnywhere, BlueprintSetter=SetAutoRegisterWithBudgetAllocator, Category = Budgeting)
	uint8 bAutoRegisterWithBudgetAllocator : 1;

	/** Whether this component should automatically calculate its significance (rather than some external system pushing the significance to it) */
	UPROPERTY(EditAnywhere, Category = Budgeting)
	uint8 bAutoCalculateSignificance : 1;

	/** Whether this component should use its owning actor's rendered state to determine visibility. If this is not set then the component's visibility will be used */
	UPROPERTY(EditAnywhere, Category = Budgeting, AdvancedDisplay)
	uint8 bShouldUseActorRenderedFlag : 1;
};

#undef UE_API
